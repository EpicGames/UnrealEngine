{
	"FileVersion": 3,
	"Version": 1.0,
	"VersionName": "1.0",
	"FriendlyName": "Pixel Capture",
	"Description": "Framework for capturing pixel buffers in other formats while allowing for disconnected produce/consume rates.",
	"Category": "Graphics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Platforms/PixelStreaming/index.html",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PixelCapture",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"TargetDenyList": [ "Server" ]
		},
		{
            "Name": "PixelCaptureShaders",
            "Type": "Runtime",
            "LoadingPhase": "PostConfigInit",
            "PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
            "TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		}
	]
}
