// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/ColorUtils.ush"

Texture2D SourceTexture;
SamplerState SourceSampler;
float2 SourceDimensions;
RWTexture2D<float> OutputY;
RWTexture2D<float> OutputU;
RWTexture2D<float> OutputV;

static const float4x4 ColorTransform = float4x4( 0.257,  0.504,  0.098, 0.063,
                                                -0.148, -0.291,  0.439, 0.500,
                                                 0.439, -0.368, -0.071, 0.500,
                                                 0.000,  0.000,  0.000, 1.000);

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void ExtractI420(uint3 Gid : SV_GroupID,
                 uint3 DTid : SV_DispatchThreadID,
                 uint3 GTid : SV_GroupThreadID,
                 uint GI : SV_GroupIndex)
{
    // i tried sampling here so we would only need to transform the colours
    // once, but it resulted in a very blurry image with the Y component sampled.
    // sampling the UV values but picking the Y value seems to give the best
    // result.

    float3 yuv00 = RgbToYuv(SourceTexture[DTid.xy].rgb, ColorTransform, 0);
    OutputY[DTid.xy] = yuv00.x;

    if (DTid.x % 2 == 0 && DTid.y % 2)
    {
        float2 uv = (DTid.xy + float2(1, 1)) / SourceDimensions;
        float3 sampledColor = SourceTexture.SampleLevel(SourceSampler, uv, 0).rgb;
        float3 yuvSampled = RgbToYuv(sampledColor, ColorTransform, 0);

        OutputU[DTid.xy / 2] = yuvSampled.y;
        OutputV[DTid.xy / 2] = yuvSampled.z;
    }
}
