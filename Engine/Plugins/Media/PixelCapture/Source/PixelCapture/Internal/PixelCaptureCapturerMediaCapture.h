// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StripedMap.h"
#include "MediaCapture.h"
#include "MediaOutput.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureCapturer.h"

#include "PixelCaptureCapturerMediaCapture.generated.h"

#define UE_API PIXELCAPTURE_API

UCLASS(MinimalAPI, BlueprintType)
class UPixelCaptureMediaOuput : public UMediaOutput
{
	GENERATED_BODY()

public:
	void									 SetRequestedSize(FIntPoint InRequestedSize) { RequestedSize = InRequestedSize; }
	virtual FIntPoint						 GetRequestedSize() const override { return RequestedSize; }
	virtual EPixelFormat					 GetRequestedPixelFormat() const override { return EPixelFormat::PF_B8G8R8A8; }
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override { return EMediaCaptureConversionOperation::CUSTOM; }

private:
	FIntPoint RequestedSize;
};

UCLASS(MinimalAPI, BlueprintType)
class UPixelCaptureMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

public:
	void AddOutputFrame(uint64 InFrameNumberRenderThread, TSharedPtr<IPixelCaptureOutputFrame> InOutputFrame)
	{
		OutputFrames.Add(InFrameNumberRenderThread, InOutputFrame);
	}

	void RemoveOutputFrame(uint64 InFrameNumberRenderThread)
	{
		OutputFrames.Remove(InFrameNumberRenderThread);
	}

	void SetFormat(int32 InFormat)
	{
		Format = InFormat;
	}

	void SetIsSRGB(bool bInIsSRGB)
	{
		bIsSRGB = bInIsSRGB;
	}

	DECLARE_EVENT_OneParam(UPixelCaptureMediaCapture, FOnCaptureComplete, TSharedPtr<IPixelCaptureOutputFrame>);
	FOnCaptureComplete OnCaptureComplete;

protected:
	virtual bool InitializeCapture() override
	{
		SetState(EMediaCaptureState::Capturing);
		return true;
	}

	UE_API virtual void OnRHIResourceCaptured_AnyThread(
		const FCaptureBaseData&								   BaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UserData,
		FTextureRHIRef										   Texture) override;

	UE_API virtual void OnRHIResourceCaptured_RenderingThread(
		FRHICommandListImmediate& RHICmdList,
		const FCaptureBaseData& BaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UserData,
		FTextureRHIRef Texture) override;

	UE_API virtual void OnFrameCaptured_AnyThread(
		const FCaptureBaseData&								   BaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UserData,
		const FMediaCaptureResourceData&					   ResourceData) override;

	UE_API virtual void OnCustomCapture_RenderingThread(
		FRDGBuilder&										   GraphBuilder,
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		FRDGTextureRef										   InSourceTexture,
		FRDGTextureRef										   OutputTexture,
		const FRHICopyTextureInfo&							   CopyInfo,
		FVector2D											   CropU,
		FVector2D											   CropV) override;

	UE_API virtual void OnFrameCaptured_RenderingThread(
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		void*												   InBuffer,
		int32												   Width,
		int32												   Height,
		int32												   BytesPerRow) override;

	virtual bool ShouldCaptureRHIResource() const override
	{
		return Format == PixelCaptureBufferFormat::FORMAT_RHI;
	}

	virtual bool SupportsAnyThreadCapture() const
	{
#if PLATFORM_APPLE
		// On Mac, Media Capture must capture cpu frames on the render thread.
		// This is because MediaCapture Lock_Unsafe uses RHIMapStagingSurface
		// which needs to run on the rendering thread with Metal.
		return Format == PixelCaptureBufferFormat::FORMAT_RHI;
#else  // PLATFORM_APPLE
		return true;
#endif // PLATFORM_APPLE
	}

	UE_API virtual ETextureCreateFlags GetOutputTextureFlags() const override;
	UE_API virtual void				WaitForGPU(FRHITexture* InRHITexture) override;
private:
	int32 Format = PixelCaptureBufferFormat::FORMAT_UNKNOWN;

	bool bIsSRGB = false;

	TStripedMap<32, uintptr_t, FGPUFenceRHIRef> Fences;

	//  SourceFrameNumberRenderThread, OutputFrame
	TStripedMap<32, uint64, TSharedPtr<IPixelCaptureOutputFrame>> OutputFrames;
};

/**
 * A MediaIO based capturer that will copy and convert RHI texture frames.
 * Input: FPixelCaptureInputFrameRHI
 * Output: FPixelCaptureOutputFrameRHI/FPixelCaptureOutputFrameI420
 */
class FPixelCaptureCapturerMediaCapture : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerMediaCapture>
{
public:
	/**
	 * Creates a new Capturer capturing the input frame at the given resolution.
	 * @param InOutputResolution The resolution of the resulting output capture.
	 */
	static UE_API TSharedPtr<FPixelCaptureCapturerMediaCapture> Create(FPixelCaptureCapturerConfig Config, int32 InFormat);
	UE_API virtual ~FPixelCaptureCapturerMediaCapture() override;

protected:
	UE_API virtual FString					  GetCapturerName() const override;
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void					  BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	uint64 LastFrameCounterRenderThread = 0;
	int32  Format = PixelCaptureBufferFormat::FORMAT_UNKNOWN;
	bool   bMediaCaptureInitialized = false;

	UE_API FPixelCaptureCapturerMediaCapture(FPixelCaptureCapturerConfig& Config, int32 InFormat);
	UE_API void InitializeMediaCapture();

	UE_API void InvalidateOutputBuffer(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);

	TObjectPtr<UPixelCaptureMediaCapture> MediaCapture = nullptr;

	TObjectPtr<UPixelCaptureMediaOuput> MediaOutput = nullptr;
};

#undef UE_API
