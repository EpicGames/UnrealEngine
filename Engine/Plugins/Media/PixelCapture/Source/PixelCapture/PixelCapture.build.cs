// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class PixelCapture : ModuleRules
	{
		public PixelCapture(ReadOnlyTargetRules Target) : base(Target)
		{
			var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

			// This is so for game projects using our public headers don't have to include extra modules they might not know about.
			PublicDependencyModuleNames.AddRange(new string[] {
				"RHI",
				"MediaIOCore",
			});

			PrivateDependencyModuleNames.AddRange(new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"PixelCaptureShaders",
				"RenderCore",
				"Renderer",
				"WebRTC",
			});
		}
	}
}
