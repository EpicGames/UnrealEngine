// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputFrameBuffer.h"

#include "PixelCapturePrivate.h"

namespace UE::PixelCapture
{
	FOutputFrameBuffer::FOutputFrameBuffer()
		: ProduceIndex(0)
		, ConsumeIndex(0)
		, MaxSize(0)
	{
	}

	void FOutputFrameBuffer::Reset(int32 InitialSize, int32 InMaxSize, const FFrameFactory& InFrameFactory)
	{
		checkf(InitialSize > 1, TEXT("InitialSize must be larger than one."));
		FrameFactory = InFrameFactory;
		BufferRing.Empty();
		MaxSize = InMaxSize;
		Grow(InitialSize);
		ProduceIndex = 0;
		ConsumeIndex = 1;
	}

	TSharedPtr<IPixelCaptureOutputFrame> FOutputFrameBuffer::LockProduceBuffer()
	{
		checkf(BufferRing.Num() > 0, TEXT("Ring is empty."));

		bool bIsFull = true;

		// find an unreferenced buffer. by adding one to produce index repeatedly
		for (int i = 0; i < BufferRing.Num(); ++i)
		{
			ProduceIndex = (ProduceIndex + 1) % BufferRing.Num();
			if (ProduceIndex == ConsumeIndex)
			{
				continue;
			}
			if (BufferRing[ProduceIndex].IsUnique())
			{
				bIsFull = false;
				break;
			}
		}

		if (bIsFull)
		{
			if (BufferRing.Num() < MaxSize)
			{
				// add a new element and use it.
				Grow(BufferRing.Num() + 1);
				ProduceIndex = BufferRing.Num() - 1;
			}
			else
			{
				return nullptr;
			}
		}

		return BufferRing[ProduceIndex];
	}

	bool FOutputFrameBuffer::ReleaseProduceBuffer(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
	{
		for (int i = 0; i < BufferRing.Num(); ++i)
		{
			if (BufferRing[i] == OutputBuffer)
			{
				// Set the consumer index to the most recently available output buffer.
				// This enables multiple produce buffers to be mid capture only consume
				// the most recently finished output buffer
				ConsumeIndex = i;
				return true;
			}
		}
		UE_LOGF(LogPixelCapture, Warning, "Failed to release produce buffer.");
		return false;
	}

	TSharedPtr<IPixelCaptureOutputFrame> FOutputFrameBuffer::GetConsumeBuffer()
	{
		checkf(ConsumeIndex < BufferRing.Num(), TEXT("Consume index is outside valid range. Was Reset() called?"));
		return BufferRing[ConsumeIndex];
	}

	void FOutputFrameBuffer::Grow(int32 NewSize)
	{
		const int32 OldSize = BufferRing.Num();
		checkf(NewSize <= MaxSize, TEXT("Ring is at max size."));
		checkf(OldSize < NewSize, TEXT("Ring should only increase in size."));
		BufferRing.SetNum(NewSize);
		for (int32 i = OldSize; i < NewSize; ++i)
		{
			BufferRing[i] = FrameFactory();
		}
	}
} // namespace UE::PixelCapture
