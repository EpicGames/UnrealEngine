// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureOutputFrame.h"

namespace UE::PixelCapture
{
	/**
	 * A Single Producer Multiple Consumer ring buffer.
	 * Used by the capture process where there is a single source of captured frames
	 * that are then consumed by multiple users.
	 * We use the ref counting on TSharedPtr to check if the buffer is being used
	 * by anyone and if not we use it as a produce buffer.
	 * Once the produce is done we mark that buffer as the next consume buffer so
	 * that GetConsumeBuffer will always consume the most recently produced buffer.
	 */
	class FOutputFrameBuffer
	{
	public:
		using FFrameFactory = TFunction<TSharedPtr<IPixelCaptureOutputFrame>()>;

		FOutputFrameBuffer();

		/**
		 * Resets the ring buffer to initial state. Must be called at least once
		 * before use.
		 * @param InitialSize The starting number of available buffers.
		 * @param InMaxSize The maximum number of buffers to grow to.
		 * @param InFrameFactory A function object that is used to create new buffers when growing.
		 */
		void Reset(int32 InitialSize, int32 InMaxSize, const FFrameFactory& InFrameFactory);

		/**
		 * Gets the next available produce buffer. Might cause the buffer to grow by one up
		 * to MaxSize.
		 * @return A buffer ready to write to or null in the case of a full ring.
		 */
		TSharedPtr<IPixelCaptureOutputFrame> LockProduceBuffer();

		/**
		 * Indicates that the previously acquired produce buffer has been filled and
		 * ready to consume. Updates the consume position to this buffer.
		 */
		bool ReleaseProduceBuffer(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);

		/**
		 * Gets the newest buffer ready to consume.
		 * @return Will always return a buffer ready to be consumed.
		 */
		TSharedPtr<IPixelCaptureOutputFrame> GetConsumeBuffer();

	private:
		FFrameFactory FrameFactory;
		TArray<TSharedPtr<IPixelCaptureOutputFrame>> BufferRing;
		TAtomic<int32> ProduceIndex;
		TAtomic<int32> ConsumeIndex;
		int32 MaxSize;

		void Grow(int32 NewSize);
	};
} // namespace UE::PixelCapture
