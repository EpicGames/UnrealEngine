// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureBufferI010.h"

namespace
{
	int CalcI010Size(int StrideY, int StrideUV, int Height)
	{
		return 2 * (StrideY * Height + (StrideUV + StrideUV) * ((Height + 1) / 2));
	}
} // namespace

FPixelCaptureBufferI010::FPixelCaptureBufferI010(int InWidth, int InHeight)
	: Width(InWidth)
	, Height(InHeight)
	, StrideY(InWidth)
	, StrideUV((InWidth + 1) / 2)
{
	Data.SetNum(CalcI010Size(StrideY, StrideUV, Height));
}

const uint8_t* FPixelCaptureBufferI010::GetData() const
{
	return reinterpret_cast<const uint8_t*>(Data.GetData());
}

const uint16_t* FPixelCaptureBufferI010::GetDataY() const
{
	return Data.GetData();
}

const uint16_t* FPixelCaptureBufferI010::GetDataU() const
{
	return GetDataY() + GetDataSizeY();
}

const uint16_t* FPixelCaptureBufferI010::GetDataV() const
{
	return GetDataU() + GetDataSizeUV();
}

uint8_t* FPixelCaptureBufferI010::GetMutableData()
{
	return reinterpret_cast<uint8_t*>(Data.GetData());
}

uint16_t* FPixelCaptureBufferI010::GetMutableDataY()
{
	return Data.GetData();
}

uint16_t* FPixelCaptureBufferI010::GetMutableDataU()
{
	return GetMutableDataY() + GetDataSizeY();
}

uint16_t* FPixelCaptureBufferI010::GetMutableDataV()
{
	return GetMutableDataU() + GetDataSizeUV();
}

int FPixelCaptureBufferI010::GetDataSizeY() const
{
	return StrideY * Height;
}

int FPixelCaptureBufferI010::GetDataSizeUV() const
{
	return StrideUV * ((Height + 1) / 2);
}
