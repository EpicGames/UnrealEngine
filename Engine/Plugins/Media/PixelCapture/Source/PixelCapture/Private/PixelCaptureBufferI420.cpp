// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureBufferI420.h"

namespace
{
	int CalcI420Size(int StrideY, int StrideUV, int Height)
	{
		return StrideY * Height + ((StrideUV + StrideUV) * ((Height + 1) / 2));
	}
}

FPixelCaptureBufferI420::FPixelCaptureBufferI420(int InWidth, int InHeight)
	: Width(InWidth)
	, Height(InHeight)
	, StrideY(InWidth)
	, StrideUV((InWidth + 1) / 2)
{
	Data.SetNum(CalcI420Size(StrideY, StrideUV, Height));
}

const uint8_t* FPixelCaptureBufferI420::GetData() const
{
	return Data.GetData();
}

const uint8_t* FPixelCaptureBufferI420::GetDataY() const
{
	return GetData();
}

const uint8_t* FPixelCaptureBufferI420::GetDataU() const
{
	return GetDataY() + GetDataSizeY();
}

const uint8_t* FPixelCaptureBufferI420::GetDataV() const
{
	return GetDataU() + GetDataSizeUV();
}

uint8_t* FPixelCaptureBufferI420::GetMutableData()
{
	return Data.GetData();
}

uint8_t* FPixelCaptureBufferI420::GetMutableDataY()
{
	return GetMutableData();
}

uint8_t* FPixelCaptureBufferI420::GetMutableDataU()
{
	return GetMutableDataY() + GetDataSizeY();
}

uint8_t* FPixelCaptureBufferI420::GetMutableDataV()
{
	return GetMutableDataU() + GetDataSizeUV();
}

int FPixelCaptureBufferI420::GetDataSizeY() const
{
	return StrideY * Height;
}

int FPixelCaptureBufferI420::GetDataSizeUV() const
{
	return StrideUV * ((Height + 1) / 2);
}
