// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureBufferNV12.h"

namespace
{
	int CalcNV12Size(int StrideY, int StrideUV, int Height)
	{
		return StrideY * Height + (StrideUV * ((Height + 1) / 2));
	}
}

FPixelCaptureBufferNV12::FPixelCaptureBufferNV12(int InWidth, int InHeight)
	: Width(InWidth)
	, Height(InHeight)
	, StrideY(InWidth)
	, StrideUV(InWidth)
{
	Data.SetNum(CalcNV12Size(StrideY, StrideUV, Height));
}

const uint8_t* FPixelCaptureBufferNV12::GetData() const
{
	return Data.GetData();
}

const uint8_t* FPixelCaptureBufferNV12::GetDataY() const
{
	return GetData();
}

const uint8_t* FPixelCaptureBufferNV12::GetDataUV() const
{
	return GetDataY() + GetDataSizeY();
}

uint8_t* FPixelCaptureBufferNV12::GetMutableData()
{
	return Data.GetData();
}

uint8_t* FPixelCaptureBufferNV12::GetMutableDataY()
{
	return GetMutableData();
}

uint8_t* FPixelCaptureBufferNV12::GetMutableDataUV()
{
	return GetMutableDataY() + GetDataSizeY();
}

int FPixelCaptureBufferNV12::GetDataSizeY() const
{
	return StrideY * Height;
}

int FPixelCaptureBufferNV12::GetDataSizeUV() const
{
	return StrideUV * ((Height + 1) / 2);
}
