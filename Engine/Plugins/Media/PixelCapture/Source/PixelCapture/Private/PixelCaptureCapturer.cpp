// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturer.h"
#include "OutputFrameBuffer.h"
#include "PixelCapturePrivate.h"

namespace UE::PixelCapture
{
	void MarkCPUWorkStart(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
	{
		if (!OutputBuffer)
		{
			UE_LOGF(LogPixelCapture, Warning, "Unable to update metadata on a null output buffer!");
			return;
		}

		OutputBuffer->Metadata.CaptureProcessCPUStartCycles = FPlatformTime::Cycles64();
	}

	void MarkCPUWorkEnd(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
	{
		if (!OutputBuffer)
		{
			UE_LOGF(LogPixelCapture, Warning, "Unable to update metadata on a null output buffer!");
			return;
		}

		const uint64 CyclesNow = FPlatformTime::Cycles64();

		if (OutputBuffer->Metadata.CaptureProcessCPUStartCycles == 0)
		{
			OutputBuffer->Metadata.CaptureProcessCPUStartCycles = CyclesNow;
		}

		OutputBuffer->Metadata.CaptureProcessCPUEndCycles = CyclesNow;
		OutputBuffer->Metadata.CaptureProcessGPUEnqueueStartCycles = CyclesNow;
	}

	void MarkGPUWorkStart(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
	{
		if (!OutputBuffer)
		{
			UE_LOGF(LogPixelCapture, Warning, "Unable to update metadata on a null output buffer!");
			return;
		}

		const uint64 CyclesNow = FPlatformTime::Cycles64();

		if (OutputBuffer->Metadata.CaptureProcessGPUEnqueueStartCycles == 0)
		{
			OutputBuffer->Metadata.CaptureProcessGPUEnqueueStartCycles = CyclesNow;
		}

		OutputBuffer->Metadata.CaptureProcessGPUEnqueueEndCycles = CyclesNow;
		OutputBuffer->Metadata.CaptureProcessGPUStartCycles = CyclesNow;
	}

	void MarkGPUWorkEnd(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
	{
		if (!OutputBuffer)
		{
			UE_LOGF(LogPixelCapture, Warning, "Unable to update metadata on a null output buffer!");
			return;
		}

		const uint64 CyclesNow = FPlatformTime::Cycles64();

		if (OutputBuffer->Metadata.CaptureProcessGPUStartCycles == 0)
		{
			OutputBuffer->Metadata.CaptureProcessGPUStartCycles = CyclesNow;
		}

		OutputBuffer->Metadata.CaptureProcessGPUEndCycles = CyclesNow;
		OutputBuffer->Metadata.CaptureProcessPostGPUStartCycles = CyclesNow;
	}
} // namespace UE::PixelCapture

FPixelCaptureCapturer::FPixelCaptureCapturer(FPixelCaptureCapturerConfig Config)
	: Config(Config)
{
}

// defined here so we can delete the FOutputFrameBuffer
FPixelCaptureCapturer::~FPixelCaptureCapturer() = default;

TSharedPtr<IPixelCaptureOutputFrame> FPixelCaptureCapturer::ReadOutput()
{
	if (bHasOutput)
	{
		return Buffer->GetConsumeBuffer();
	}
	return nullptr;
}

void FPixelCaptureCapturer::Capture(const IPixelCaptureInputFrame& InputFrame)
{
	if (IsBusy())
	{
		return;
	}

	if (!IsInitialized())
	{
		Initialize(InputFrame.GetWidth(), InputFrame.GetHeight());
	}

	checkf(InputFrame.GetWidth() == ExpectedInputWidth && InputFrame.GetHeight() == ExpectedInputHeight, TEXT("Capturer input resolution changes are not supported"));

	TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer = Buffer->LockProduceBuffer();
	if (!OutputBuffer)
	{
		UE_LOGF(LogPixelCapture, Error, "Failed to obtain a produce buffer.");
		return;
	}

	InitMetadata(InputFrame.Metadata.Copy(), OutputBuffer);

	BeginProcess(InputFrame, OutputBuffer);
}

void FPixelCaptureCapturer::Initialize(int32 InputWidth, int32 InputHeight)
{
	Buffer = MakeUnique<UE::PixelCapture::FOutputFrameBuffer>();
	Buffer->Reset(3, 32, [this, InputWidth, InputHeight]() { return TSharedPtr<IPixelCaptureOutputFrame>(CreateOutputBuffer(InputWidth, InputHeight)); });
	ExpectedInputWidth = InputWidth;
	ExpectedInputHeight = InputHeight;
	bHasOutput = false;
	bInitialized = true;
}

void FPixelCaptureCapturer::InitMetadata(FPixelCaptureFrameMetadata Metadata, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	if (!OutputBuffer)
	{
		UE_LOGF(LogPixelCapture, Warning, "Unable to update metadata on a null output buffer!");
		return;
	}

	Metadata.Id = FrameId.Increment();
	Metadata.ProcessName = GetCapturerName();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OutputBuffer->Metadata = Metadata;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Reset all the capture timestamps
	OutputBuffer->Metadata.CaptureStartCyles = 0;
	OutputBuffer->Metadata.CaptureEndCyles = 0;
	OutputBuffer->Metadata.CaptureProcessCPUStartCycles = 0;
	OutputBuffer->Metadata.CaptureProcessCPUEndCycles = 0;
	OutputBuffer->Metadata.CaptureProcessGPUEnqueueStartCycles = 0;
	OutputBuffer->Metadata.CaptureProcessGPUEnqueueEndCycles = 0;
	OutputBuffer->Metadata.CaptureProcessGPUStartCycles = 0;
	OutputBuffer->Metadata.CaptureProcessGPUEndCycles = 0;
	OutputBuffer->Metadata.CaptureProcessPostGPUStartCycles = 0;
	OutputBuffer->Metadata.CaptureProcessPostGPUEndCycles = 0;

	OutputBuffer->Metadata.CaptureStartCyles = FPlatformTime::Cycles64();
}

void FPixelCaptureCapturer::FinalizeMetadata(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	if (!OutputBuffer)
	{
		UE_LOGF(LogPixelCapture, Warning, "Unable to update metadata on a null output buffer!");
		return;
	}

	const uint64 CyclesNow = FPlatformTime::Cycles64();

	if (OutputBuffer->Metadata.CaptureProcessPostGPUEndCycles == 0)
	{
		OutputBuffer->Metadata.CaptureProcessPostGPUEndCycles = CyclesNow;
	}

	OutputBuffer->Metadata.CaptureEndCyles = CyclesNow;
}

void FPixelCaptureCapturer::EndProcess(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	if (!OutputBuffer)
	{
		UE_LOGF(LogPixelCapture, Warning, "Unable to end process on a null output buffer!");
		return;
	}

	FinalizeMetadata(OutputBuffer);

	if (Buffer->ReleaseProduceBuffer(OutputBuffer))
	{
		bHasOutput = true;
		OnComplete.Broadcast();
	}
	else
	{
		bHasOutput = false;
	}
}
