// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerI420.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureInputFrameI420.h"
#include "PixelCaptureBufferFormat.h"

TSharedPtr<FPixelCaptureCapturerI420> FPixelCaptureCapturerI420::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerI420>(new FPixelCaptureCapturerI420(Config));
}

FPixelCaptureCapturerI420::FPixelCaptureCapturerI420(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

void FPixelCaptureCapturerI420::Initialize(int32 InputWidth, int32 InputHeight)
{
	FPixelCaptureCapturer::Initialize(InputWidth, InputHeight);
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerI420::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	return new FPixelCaptureOutputFrameI420(MakeShared<FPixelCaptureBufferI420>(InputWidth, InputHeight));
}

void FPixelCaptureCapturerI420::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_I420), TEXT("Incorrect source frame coming into frame capture process, expected FORMAT_I420 frame."));

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

	// just copy the input frame
	const FPixelCaptureInputFrameI420&		 SourceFrame = StaticCast<const FPixelCaptureInputFrameI420&>(InputFrame);
	TSharedPtr<FPixelCaptureOutputFrameI420> OutputI420Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameI420>(OutputBuffer);
	OutputI420Buffer->GetI420Buffer()->Copy(*SourceFrame.GetBuffer());

	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	EndProcess(OutputBuffer);
	SetIsBusy(false);
}
