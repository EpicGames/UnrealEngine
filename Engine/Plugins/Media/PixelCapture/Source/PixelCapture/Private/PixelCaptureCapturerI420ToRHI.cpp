// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerI420ToRHI.h"
#include "PixelCaptureInputFrameI420.h"
#include "PixelCaptureInputFrameNV12.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"

#include "Async/Async.h"
#include "RenderingThread.h"
#include "RHICommandList.h"
#include "RHITypes.h"

#include "libyuv/convert.h"
#include "libyuv/video_common.h"
#include "common_video/libyuv/include/webrtc_libyuv.h"

TSharedPtr<FPixelCaptureCapturerI420ToRHI> FPixelCaptureCapturerI420ToRHI::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerI420ToRHI>(new FPixelCaptureCapturerI420ToRHI(Config));
}

FPixelCaptureCapturerI420ToRHI::FPixelCaptureCapturerI420ToRHI(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

FPixelCaptureCapturerI420ToRHI::~FPixelCaptureCapturerI420ToRHI()
{
	if (ARGBBuffer)
	{
		delete[] ARGBBuffer;
	}
}

void FPixelCaptureCapturerI420ToRHI::Initialize(int32 InputWidth, int32 InputHeight)
{
	Config.OutputResolution = { InputWidth, InputHeight };

	const uint32_t Size = webrtc::CalcBufferSize(webrtc::VideoType::kARGB, Config.OutputResolution.X, Config.OutputResolution.Y);
	ARGBBuffer = new uint8_t[Size];

	FPixelCaptureCapturer::Initialize(InputWidth, InputHeight);
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerI420ToRHI::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	// OpenGL needs PF_R8G8B8A8
	EPixelFormat PixelFormat = RHIGetInterfaceType() == ERHIInterfaceType::OpenGL ? PF_R8G8B8A8 : PF_B8G8R8A8;

	FRHITextureCreateDesc TextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerI420ToRHI Texture"), InputWidth, InputHeight, PixelFormat)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::CPUWritable | ETextureCreateFlags::RenderTargetable)
			.SetInitialState(ERHIAccess::Present)
			.DetermineInititialState();

	if (Config.bIsSRGB)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::SRGB);
	}

#if !PLATFORM_ANDROID
	// TODO (william.belcher): Android doesn't seem to like the External/Shared flags - investigate further
	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::External);
	}
	else
	{
		TextureDesc.AddFlags(ETextureCreateFlags::Shared);
	}
#endif

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	return new FPixelCaptureOutputFrameRHI(RHICmdList.CreateTexture(TextureDesc));
}

void FPixelCaptureCapturerI420ToRHI::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	if (InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_I420))
	{
		UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

		const FPixelCaptureInputFrameI420&		SourceFrame = StaticCast<const FPixelCaptureInputFrameI420&>(InputFrame);
		TSharedPtr<FPixelCaptureBufferI420>		I420Buffer = SourceFrame.GetBuffer();
		TSharedPtr<FPixelCaptureOutputFrameRHI> OutputTexture = StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputBuffer);

		uint32 FourCC = (RHIGetInterfaceType() == ERHIInterfaceType::OpenGL) ? libyuv::FOURCC_ABGR : libyuv::FOURCC_ARGB;

		// convert the incoming i420 to argb
		libyuv::ConvertFromI420(
			I420Buffer->GetDataY(), I420Buffer->GetStrideY(),
			I420Buffer->GetDataU(), I420Buffer->GetStrideUV(),
			I420Buffer->GetDataV(), I420Buffer->GetStrideUV(),
			ARGBBuffer, 0,
			I420Buffer->GetWidth(), I420Buffer->GetHeight(),
			FourCC);

		TWeakPtr<FPixelCaptureCapturerI420ToRHI> WeakSelf = StaticCastSharedRef<FPixelCaptureCapturerI420ToRHI>(AsShared());
		ENQUEUE_RENDER_COMMAND(I420toRHI)
		([WeakSelf, OutputTexture](FRHICommandListImmediate& RHICmdList) -> void {
			if (TSharedPtr<FPixelCaptureCapturerI420ToRHI> Self = WeakSelf.Pin())
			{
				UE::PixelCapture::MarkGPUWorkStart(OutputTexture);

				// update our staging texture
				FUpdateTextureRegion2D Region(0, 0, 0, 0, Self->Config.OutputResolution.X, Self->Config.OutputResolution.Y);
				RHICmdList.UpdateTexture2D(OutputTexture->GetFrameTexture(), 0, Region, Self->Config.OutputResolution.X * 4, Self->ARGBBuffer);

				UE::PixelCapture::MarkGPUWorkEnd(OutputTexture);
				Self->EndProcess(OutputTexture);
				Self->SetIsBusy(false);
			} });

		UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);
	}
	else
	{
		unimplemented();
	}
}
