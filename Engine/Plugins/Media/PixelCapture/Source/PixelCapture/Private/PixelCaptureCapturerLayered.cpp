// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerLayered.h"
#include "PixelCaptureCapturer.h"
#include "PixelCapturePrivate.h"
#include "Misc/ScopeLock.h"

TSharedPtr<FPixelCaptureCapturerLayered> FPixelCaptureCapturerLayered::Create(IPixelCaptureCapturerSource* InCapturerSource, int32 InDestinationFormat, TArray<FIntPoint> InOutputResolutions)
{
	TSharedPtr<FPixelCaptureCapturerLayered> LayeredCapturer = TSharedPtr<FPixelCaptureCapturerLayered>(new FPixelCaptureCapturerLayered(InCapturerSource, InDestinationFormat));

	for (FIntPoint OutputResolution : InOutputResolutions)
	{
		TSharedPtr<FPixelCaptureCapturer> CaptureProcess = InCapturerSource->CreateCapturer(InDestinationFormat, OutputResolution);
		CaptureProcess->OnComplete.AddSP(LayeredCapturer.ToSharedRef(), &FPixelCaptureCapturerLayered::OnCaptureComplete);
		LayeredCapturer->LayerCapturers.Add(OutputResolution, CaptureProcess);
	}

	return LayeredCapturer;
}

TSharedPtr<FPixelCaptureCapturerLayered> FPixelCaptureCapturerLayered::Create(IPixelCaptureCapturerSource* InCapturerSource, int32 InDestinationFormat, TArray<float> InLayerScales)
{
	return TSharedPtr<FPixelCaptureCapturerLayered>(new FPixelCaptureCapturerLayered(InCapturerSource, InDestinationFormat));
}

FPixelCaptureCapturerLayered::FPixelCaptureCapturerLayered(IPixelCaptureCapturerSource* InCapturerSource, int32 InDestinationFormat)
	: CapturerSource(InCapturerSource)
	, DestinationFormat(InDestinationFormat)
{
}

TSharedPtr<IPixelCaptureOutputFrame> FPixelCaptureCapturerLayered::ReadOutput(FIntPoint LayerSize)
{
	FScopeLock LayersLock(&LayersGuard);
	if (LayerCapturers.Contains(LayerSize))
	{
		return LayerCapturers[LayerSize]->ReadOutput();
	}

	// A user has requested output for a resolution that hasn't been captured yet. Create a capturer for that resolution
	TSharedPtr<FPixelCaptureCapturer> CaptureProcess = CapturerSource->CreateCapturer(DestinationFormat, LayerSize);
	CaptureProcess->OnComplete.AddSP(AsShared(), &FPixelCaptureCapturerLayered::OnCaptureComplete);
	LayerCapturers.Add(LayerSize, CaptureProcess);

	return nullptr;
}

void FPixelCaptureCapturerLayered::OnCaptureComplete()
{
	OnComplete.Broadcast();
}

void FPixelCaptureCapturerLayered::Capture(const IPixelCaptureInputFrame& SourceFrame)
{
	// work on a temp list so we dont over lock
	TArray<TSharedPtr<FPixelCaptureCapturer>> TempLayerCapturers;
	{
		FScopeLock LayersLock(&LayersGuard);
		LayerCapturers.GenerateValueArray(TempLayerCapturers);
	}

	// capture the frame for encoder use
	for (auto& LayerCapturer : TempLayerCapturers)
	{
		LayerCapturer->Capture(SourceFrame);
	}
}
