// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerMediaCapture.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureUtils.h"
#include "PixelCapturePrivate.h"

#include "Async/Async.h"

#include "libyuv/convert.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelCaptureCapturerMediaCapture)

BEGIN_SHADER_PARAMETER_STRUCT(FWriteFenceParameters, )
SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
END_SHADER_PARAMETER_STRUCT()

void UPixelCaptureMediaCapture::OnRHIResourceCaptured_AnyThread(
	const FCaptureBaseData&								   BaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UserData,
	FTextureRHIRef										   Texture)
{
	check(Format == PixelCaptureBufferFormat::FORMAT_RHI);

	TSharedPtr<IPixelCaptureOutputFrame> OutputFrame;
	if (!OutputFrames.RemoveAndCopyValue(BaseData.SourceFrameNumberRenderThread, OutputFrame) || !OutputFrame)
	{
		UE_LOGFMT(LogPixelCapture, Warning, "UPixelCaptureMediaCapture::OnRHIResourceCaptured_AnyThread: No output frame set for frame number {0}!", BaseData.SourceFrameNumberRenderThread);
		return;
	}

	StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputFrame)->SetFrameTexture(Texture);

	OnCaptureComplete.Broadcast(OutputFrame);
}

void UPixelCaptureMediaCapture::OnRHIResourceCaptured_RenderingThread(
	FRHICommandListImmediate&							   RHICmdList,
	const FCaptureBaseData&								   BaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UserData,
	FTextureRHIRef										   Texture)
{
	OnRHIResourceCaptured_AnyThread(BaseData, UserData, Texture);
}

void UPixelCaptureMediaCapture::OnFrameCaptured_AnyThread(
	const FCaptureBaseData&								   BaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UserData,
	const FMediaCaptureResourceData&					   ResourceData)
{
	check(Format == PixelCaptureBufferFormat::FORMAT_I420);

	TSharedPtr<IPixelCaptureOutputFrame> OutputFrame;
	if (!OutputFrames.RemoveAndCopyValue(BaseData.SourceFrameNumberRenderThread, OutputFrame) || !OutputFrame)
	{
		UE_LOGFMT(LogPixelCapture, Warning, "UPixelCaptureMediaCapture::OnFrameCaptured_AnyThread: No output frame set for frame number {0}!", BaseData.SourceFrameNumberRenderThread);
		return;
	}

	TSharedPtr<FPixelCaptureBufferI420> I420Buffer = MakeShared<FPixelCaptureBufferI420>(ResourceData.Width, ResourceData.Height);
	libyuv::ARGBToI420(
		static_cast<uint8*>(ResourceData.Buffer),
		ResourceData.BytesPerRow,
		I420Buffer->GetMutableDataY(),
		I420Buffer->GetStrideY(),
		I420Buffer->GetMutableDataU(),
		I420Buffer->GetStrideUV(),
		I420Buffer->GetMutableDataV(),
		I420Buffer->GetStrideUV(),
		I420Buffer->GetWidth(),
		I420Buffer->GetHeight());

	StaticCastSharedPtr<FPixelCaptureOutputFrameI420>(OutputFrame)->SetI420Buffer(I420Buffer);

	OnCaptureComplete.Broadcast(OutputFrame);
}

void UPixelCaptureMediaCapture::OnFrameCaptured_RenderingThread(
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	void*												   InBuffer,
	int32												   Width,
	int32												   Height,
	int32												   BytesPerRow)
{
	FMediaCaptureResourceData ResourceData{ .Buffer = InBuffer, .Width = Width, .Height = Height, .BytesPerRow = BytesPerRow };
	OnFrameCaptured_AnyThread(InBaseData, InUserData, ResourceData);
}

void UPixelCaptureMediaCapture::OnCustomCapture_RenderingThread(
	FRDGBuilder&										   GraphBuilder,
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	FRDGTextureRef										   InSourceTexture,
	FRDGTextureRef										   OutputTexture,
	const FRHICopyTextureInfo&							   CopyInfo,
	FVector2D											   CropU,
	FVector2D											   CropV)
{
	bool bRequiresFormatConversion = InSourceTexture->Desc.Format != OutputTexture->Desc.Format;
	if (InSourceTexture->Desc.Format == OutputTexture->Desc.Format
		&& InSourceTexture->Desc.Extent.X == OutputTexture->Desc.Extent.X
		&& InSourceTexture->Desc.Extent.Y == OutputTexture->Desc.Extent.Y)
	{
		// The formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			InSourceTexture,
			OutputTexture,
			FRDGDrawTextureInfo());
	}
	else
	{
#if PLATFORM_APPLE
		// Create a staging texture that is the same size and format as the final.
		FRDGTextureRef			   StagingTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y), OutputTexture->Desc.Format, OutputTexture->Desc.ClearValue, ETextureCreateFlags::RenderTargetable), TEXT("PixelStreamingMediaIOCapture Staging"));
		FScreenPassTextureViewport StagingViewport(StagingTexture);
#endif

		FScreenPassTextureViewport InputViewport(InSourceTexture);
		FScreenPassTextureViewport OutputViewport(OutputTexture);

		FGlobalShaderMap*			 GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		int32										  MediaConversionOperation = 0; // None
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(MediaConversionOperation);

		// Rectangle area to use from source
		const FIntRect ViewRect(FIntPoint(0, 0), InSourceTexture->Desc.Extent);

		TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
		FModifyAlphaSwizzleRgbaPS::FParameters*	 PixelShaderParameters = PixelShader->AllocateAndSetParameters(
			 GraphBuilder,
			 InSourceTexture,
#if PLATFORM_APPLE
			StagingTexture
#else
			OutputTexture
#endif
		);

		FRHIBlendState*		   BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("PixelStreamingEpicRtcMediaIOCapture Swizzle"),
			FScreenPassViewInfo(),
#if PLATFORM_APPLE
			StagingViewport,
#else
			OutputViewport,
#endif

			InputViewport,
			VertexShader,
			PixelShader,
			PixelShaderParameters);

#if PLATFORM_APPLE
		// Now we can be certain the formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			StagingTexture,
			OutputTexture,
			FRDGDrawTextureInfo());
#endif
	}

	if (UseExperimentalScheduling() && ShouldCaptureRHIResource())
	{
		TWeakObjectPtr<UPixelCaptureMediaCapture> WeakThis(this);

		FWriteFenceParameters* PassParams = GraphBuilder.AllocParameters<FWriteFenceParameters>();
		PassParams->Texture = OutputTexture;

		// clang-format off
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("UPixelCaptureMediaCapture WriteFence"), 
			PassParams,
			ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,								
			[WeakThis, PassParams](FRDGAsyncTask, FRHICommandList& RHICmdList) {
				if (!WeakThis.IsValid())
				{
					return;
				}
				
				if (TStrongObjectPtr<UPixelCaptureMediaCapture> This = WeakThis.Pin())
				{
					FGPUFenceRHIRef Fence = GDynamicRHI->RHICreateGPUFence(TEXT("UPixelCaptureMediaCapture Fence"));
					This->Fences.Add(reinterpret_cast<uintptr_t>(PassParams->Texture->GetRHI()), Fence);
					RHICmdList.WriteGPUFence(Fence);
				}
		});
		// clang-format on
	}
}

ETextureCreateFlags UPixelCaptureMediaCapture::GetOutputTextureFlags() const
{
	ETextureCreateFlags Flags = TexCreate_None;
#if PLATFORM_APPLE
	Flags |= TexCreate_CPUReadback;
#else
	Flags |= TexCreate_RenderTargetable | TexCreate_UAV;

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		Flags |= TexCreate_External;
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		Flags |= TexCreate_Shared;
	}
#endif

	if (bIsSRGB)
	{
		Flags |= TexCreate_SRGB;
	}

	return Flags;
}

void UPixelCaptureMediaCapture::WaitForGPU(FRHITexture* InRHITexture)
{
	constexpr float SleepTimeSeconds = 50 * 1E-6f;
	FGPUFenceRHIRef	 Fence;

	double MaxWaitTime = FPlatformTime::Seconds() + 0.1;
	while (!Fences.RemoveAndCopyValue(reinterpret_cast<uintptr_t>(InRHITexture), Fence))
	{
		if (FPlatformTime::Seconds() >= MaxWaitTime)
		{
			UE_LOGFMT(LogPixelCapture, Error, "UPixelCaptureMediaCapture::WaitForGPU: Timed out waiting for fence to be added in pass!");
			return;
		}

		FPlatformProcess::SleepNoStats(SleepTimeSeconds);
	}

	if (!Fence.IsValid())
	{
		UE_LOGFMT(LogPixelCapture, Error, "UPixelCaptureMediaCapture::WaitForGPU: Found fence is invalid!");
		return;
	}

	MaxWaitTime = FPlatformTime::Seconds() + 0.1;
	while (!Fence->Poll())
	{
		if (FPlatformTime::Seconds() >= MaxWaitTime)
		{
			UE_LOGFMT(LogPixelCapture, Error, "UPixelCaptureMediaCapture::WaitForGPU: Timed out waiting for fence to be written!");
			return;
		}

		FPlatformProcess::SleepNoStats(SleepTimeSeconds);
	}
}

TSharedPtr<FPixelCaptureCapturerMediaCapture> FPixelCaptureCapturerMediaCapture::Create(FPixelCaptureCapturerConfig Config, int32 InFormat)
{
	TSharedPtr<FPixelCaptureCapturerMediaCapture> Capturer = TSharedPtr<FPixelCaptureCapturerMediaCapture>(new FPixelCaptureCapturerMediaCapture(Config, InFormat));

	TWeakPtr<FPixelCaptureCapturerMediaCapture> WeakCapturer = Capturer;
	AsyncTask(ENamedThreads::GameThread, [WeakCapturer]() {
		if (TSharedPtr<FPixelCaptureCapturerMediaCapture> PinnedCapturer = WeakCapturer.Pin())
		{
			PinnedCapturer->InitializeMediaCapture();
		}
	});

	return Capturer;
}

FPixelCaptureCapturerMediaCapture::FPixelCaptureCapturerMediaCapture(FPixelCaptureCapturerConfig& Config, int32 InFormat)
	: FPixelCaptureCapturer(Config)
{
	Format = InFormat;
	if (Format != PixelCaptureBufferFormat::FORMAT_RHI && Format != PixelCaptureBufferFormat::FORMAT_I420)
	{
		UE_LOGFMT(LogPixelCapture, Warning, "FPixelCaptureCapturerMediaCapture: Invalid pixel format. Expected either FORMAT_RHI or FORMAT_I420");
		return;
	}
}

FPixelCaptureCapturerMediaCapture::~FPixelCaptureCapturerMediaCapture()
{
	// We don't need to remove mediacapture from root if engine is shutting down
	// as UE will already have killed all UObjects by this point.
	if (!IsEngineExitRequested() && MediaCapture)
	{
		MediaCapture->RemoveFromRoot();
	}
}

void FPixelCaptureCapturerMediaCapture::InitializeMediaCapture()
{
	MediaCapture = NewObject<UPixelCaptureMediaCapture>();
	MediaCapture->AddToRoot(); // prevent GC on this

	MediaOutput = NewObject<UPixelCaptureMediaOuput>();
	// Note the number of texture buffers is how many textures we have in reserve to copy into while we wait for other captures to complete
	// On slower hardware this number needs to be bigger. Testing on AWS T4 GPU's (which are sort of like min-spec for PS) we determined
	// the default number (4) is too low and will cause media capture to regularly overrun (which results in either a skipped frame or a
	// GPU flush depending on the EMediaCaptureOverrunAction option below). After testing, it was found that 8 textures (the max),
	// reduced overruns to infrequent levels on the AWS T4 GPU.
	MediaOutput->NumberOfTextureBuffers = 8;
	MediaOutput->SetRequestedSize(Config.OutputResolution);
	MediaCapture->SetMediaOutput(MediaOutput);
	MediaCapture->SetFormat(Format);
	MediaCapture->SetIsSRGB(Config.bIsSRGB);
	MediaCapture->OnCaptureComplete.AddSP(AsShared(), &FPixelCaptureCapturerMediaCapture::EndProcess);

	FMediaCaptureOptions CaptureOptions;
	CaptureOptions.bSkipFrameWhenRunningExpensiveTasks = false;
	CaptureOptions.OverrunAction = EMediaCaptureOverrunAction::Skip;
	CaptureOptions.ResizeMethod = EMediaCaptureResizeMethod::ResizeInCapturePass;
	CaptureOptions.bAutostopOnCapture = true;
	// This value needs to be >= 1 in order to be a valid configuration
	// but it isn't actually used with TryCaptureImmediate_RenderThread
	CaptureOptions.NumberOfFramesToCapture = 1;

	FRHICaptureResourceDescription ResourceDescription;
	ResourceDescription.PixelFormat = EPixelFormat::PF_B8G8R8A8;

	MediaCapture->CaptureRHITexture(ResourceDescription, CaptureOptions);

	bMediaCaptureInitialized = true;
}

FString FPixelCaptureCapturerMediaCapture::GetCapturerName() const
{
	FString FormatString = TEXT("Unknown");
	switch (Format)
	{
		case PixelCaptureBufferFormat::FORMAT_RHI:
			FormatString = TEXT("RHI");
			break;
		case PixelCaptureBufferFormat::FORMAT_I420:
			FormatString = TEXT("I420");
			break;
	}

	return FString::Printf(TEXT("MediaCapture (RHI->%s)"), *FormatString);
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerMediaCapture::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	if (Format == PixelCaptureBufferFormat::FORMAT_RHI)
	{
		return new FPixelCaptureOutputFrameRHI(nullptr);
	}
	else if (Format == PixelCaptureBufferFormat::FORMAT_I420)
	{
		return new FPixelCaptureOutputFrameI420(nullptr);
	}
	else
	{
		UE_LOGFMT(LogPixelCapture, Error, "FPixelCaptureCapturerMediaCapture: Invalid pixel format. Expected either FORMAT_RHI or FORMAT_I420");
		return nullptr;
	}
}

void FPixelCaptureCapturerMediaCapture::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	if (!bMediaCaptureInitialized)
	{
		UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);
		UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);
		UE::PixelCapture::MarkGPUWorkStart(OutputBuffer);
		UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);
		// No passes added. Invalidate the output buffer so that the encoder won't pull old data from the ring buffer
		InvalidateOutputBuffer(OutputBuffer);
		// Early out as media capture is still initializing itself. We'll capture a later frame
		EndProcess(OutputBuffer);
		return;
	}

	check(IsInRenderingThread());
	if (LastFrameCounterRenderThread == GFrameCounterRenderThread)
	{
		UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);
		UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);
		UE::PixelCapture::MarkGPUWorkStart(OutputBuffer);
		UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);
		// No passes added. Invalidate the output buffer so that the encoder won't pull old data from the ring buffer
		InvalidateOutputBuffer(OutputBuffer);
		// Early out as we've already called TryCaptureImmediate_RenderThread for this frame
		EndProcess(OutputBuffer);
		return;
	}
	LastFrameCounterRenderThread = GFrameCounterRenderThread;

	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));
	const FPixelCaptureInputFrameRHI& SourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

	// We need to ensure the output frame is valid before calling TryCaptureImmediate_RenderThread
	MediaCapture->AddOutputFrame(LastFrameCounterRenderThread, OutputBuffer);

	FRDGBuilder GraphBuilder(FRHICommandListImmediate::Get());

	// clang-format off
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UPixelCaptureMediaCapture MarkGPUWorkStart"), 
		ERDGPassFlags::NeverCull,								
		[this, OutputBuffer](FRDGAsyncTask, FRHICommandList& RHICmdList) {
			UE::PixelCapture::MarkGPUWorkStart(OutputBuffer);
		});
	// clang-format on

	bool bPassesAdded = MediaCapture->TryCaptureImmediate_RenderThread(GraphBuilder, SourceFrame.FrameTexture);

	// clang-format off
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UPixelCaptureMediaCapture MarkGPUWorkEnd"), 
		ERDGPassFlags::NeverCull,								
		[this, OutputBuffer](FRDGAsyncTask, FRHICommandList& RHICmdList) {
			UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);
		});
	// clang-format on

	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	// Even if no passes are added, we still need to call Execute
	GraphBuilder.Execute();

	if (!bPassesAdded)
	{
		// TryCaptureImmediate_RenderThread didn't add any passes so remove the latest enqueued output frame from the MediaCapture
		MediaCapture->RemoveOutputFrame(LastFrameCounterRenderThread);
		// No passes added. Invalidate the output buffer so that the encoder won't pull old data from the ring buffer
		InvalidateOutputBuffer(OutputBuffer);
		// RDG graph had no passes so we can manually call EndProcess()
		EndProcess(OutputBuffer);
	}
}

void FPixelCaptureCapturerMediaCapture::InvalidateOutputBuffer(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	if (Format == PixelCaptureBufferFormat::FORMAT_RHI)
	{
		StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputBuffer)->SetFrameTexture(nullptr);
	}
	else if (Format == PixelCaptureBufferFormat::FORMAT_I420)
	{
		StaticCastSharedPtr<FPixelCaptureOutputFrameI420>(OutputBuffer)->SetI420Buffer(nullptr);
	}
	else
	{
		UE_LOGFMT(LogPixelCapture, Error, "FPixelCaptureCapturerMediaCapture: Invalid pixel format. Expected either FORMAT_RHI or FORMAT_I420");
	}
}
