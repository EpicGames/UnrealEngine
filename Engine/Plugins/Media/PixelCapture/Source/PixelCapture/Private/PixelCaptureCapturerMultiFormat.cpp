// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerMultiFormat.h"
#include "PixelCaptureCapturer.h"
#include "PixelCapturePrivate.h"
#include "Misc/ScopeLock.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"

TSharedPtr<FPixelCaptureCapturerMultiFormat> FPixelCaptureCapturerMultiFormat::Create(IPixelCaptureCapturerSource* InCapturerSource, TArray<FIntPoint> InOutputResolutions)
{
	return TSharedPtr<FPixelCaptureCapturerMultiFormat>(new FPixelCaptureCapturerMultiFormat(InCapturerSource, InOutputResolutions));
}

TSharedPtr<FPixelCaptureCapturerMultiFormat> FPixelCaptureCapturerMultiFormat::Create(IPixelCaptureCapturerSource* InCapturerSource, TArray<float> InLayerScales)
{
	return TSharedPtr<FPixelCaptureCapturerMultiFormat>(new FPixelCaptureCapturerMultiFormat(InCapturerSource, {}));
}

FPixelCaptureCapturerMultiFormat::FPixelCaptureCapturerMultiFormat(IPixelCaptureCapturerSource* InCapturerSource, TArray<FIntPoint> InOutputResolutions)
	: CapturerSource(InCapturerSource)
	, LayerSizes(InOutputResolutions)
{
}

FPixelCaptureCapturerMultiFormat::~FPixelCaptureCapturerMultiFormat()
{
	FlushWaitingEvents();
}

void FPixelCaptureCapturerMultiFormat::Capture(const IPixelCaptureInputFrame& SourceFrame)
{
	if (FormatCapturers.Num() > 0)
	{
		// UE-173694: Need to acquire the FormatGuard before FPixelCaptureCapturerLayered acquires its LayersGuard
		// to avoid a deadlock with the EncoderQueue
		FScopeLock FormatLock(&FormatGuard);
		// iterate a temp copy so modifications to the map in Capture
		// (and their callback events) is ok.
		// Note: When considering whether the lifetime of this lock can be shorter consider the deadlock issue we had in UE-173694

		TArray<TSharedPtr<FPixelCaptureCapturerLayered>> AllFormatCapturers;
		for (auto& FormatCapturer : FormatCapturers)
		{
			AllFormatCapturers.Add(FormatCapturer.Value);
		}
		// start all the format captures
		for (auto& Capturer : AllFormatCapturers)
		{
			Capturer->Capture(SourceFrame);
		}
	}
	else
	{
		OnComplete.Broadcast();
	}
}

void FPixelCaptureCapturerMultiFormat::AddOutputFormat(int32 Format)
{
	FScopeLock FormatLock(&FormatGuard);

	if (!FormatCapturers.Contains(Format))
	{
		TArray<FIntPoint> LayersArray;
		{
			FScopeLock LayerLock(&LayersGuard);
			LayersArray = LayerSizes.Array();
		}
		TSharedPtr<FPixelCaptureCapturerLayered> NewCapturer = FPixelCaptureCapturerLayered::Create(CapturerSource, Format, LayersArray);
		NewCapturer->OnComplete.AddSP(this, &FPixelCaptureCapturerMultiFormat::OnCaptureFormatComplete, Format);
		FormatCapturers.Add(Format, NewCapturer);
	}
}

TSharedPtr<IPixelCaptureOutputFrame> FPixelCaptureCapturerMultiFormat::RequestFormat(int32 Format, FIntPoint LayerSize)
{
	{
		FScopeLock FormatLock(&FormatGuard);

		if (TSharedPtr<FPixelCaptureCapturerLayered>* FormatCapturerPtr = FormatCapturers.Find(Format))
		{
			return (*FormatCapturerPtr)->ReadOutput(LayerSize);
		}
	}

	{
		FScopeLock LayerLock(&LayersGuard);
		LayerSizes.Add(LayerSize);
		LayerSizes.Sort([](const FIntPoint& A, const FIntPoint& B) {
			return A.X * A.Y < B.X * B.Y;
		});
	}

	// if we reached here then we dont have a pipeline for the given format.
	// add it and return null
	AddOutputFormat(Format);

	return nullptr;
}

TSharedPtr<IPixelCaptureOutputFrame> FPixelCaptureCapturerMultiFormat::WaitForFormat(int32 Format, FIntPoint LayerSize, uint32 MaxWaitTime)
{
	if (TSharedPtr<IPixelCaptureOutputFrame> Frame = RequestFormat(Format, LayerSize))
	{
		return Frame;
	}

	// No current output.
	// Wait for an event to signify the format capture completed
	if (FEvent* Event = GetEventForFormat(Format))
	{
		Event->Wait(MaxWaitTime);
		FreeEvent(Format, Event);
		return RequestFormat(Format, LayerSize);
	}
	return nullptr;
}

void FPixelCaptureCapturerMultiFormat::OnDisconnected()
{
	FlushWaitingEvents();
	bDisconnected = true;
}

void FPixelCaptureCapturerMultiFormat::OnCaptureFormatComplete(int32 Format)
{
	CheckFormatEvent(Format); // checks if anything is waiting on this format
	OnComplete.Broadcast();
}

FEvent* FPixelCaptureCapturerMultiFormat::GetEventForFormat(int32 Format)
{
	FScopeLock Lock(&EventMutex);
	if (!bDisconnected)
	{
		FEvent* Event = FPlatformProcess::GetSynchEventFromPool();
		FormatEvents.Add(Format, Event);
		return Event;
	}
	return nullptr;
}

void FPixelCaptureCapturerMultiFormat::CheckFormatEvent(int32 Format)
{
	FScopeLock Lock(&EventMutex);
	if (FormatEvents.Contains(Format))
	{
		FEvent* Event = FormatEvents[Format];
		FormatEvents.Remove(Format);
		Event->Trigger();
	}
}

void FPixelCaptureCapturerMultiFormat::FreeEvent(int32 Format, FEvent* Event)
{
	FScopeLock Lock(&EventMutex);
	FormatEvents.Remove(Format);
	FPlatformProcess::ReturnSynchEventToPool(Event);
}

void FPixelCaptureCapturerMultiFormat::FlushWaitingEvents()
{
	FScopeLock Lock(&EventMutex);
	for (auto& KeyValue : FormatEvents)
	{
		KeyValue.Value->Trigger();
	}
	FormatEvents.Empty();
}