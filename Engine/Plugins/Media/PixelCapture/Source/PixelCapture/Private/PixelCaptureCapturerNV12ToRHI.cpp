// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerNV12ToRHI.h"
#include "PixelCaptureInputFrameI420.h"
#include "PixelCaptureInputFrameNV12.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"

#include "Async/Async.h"
#include "DynamicRHI.h"
#include "RenderingThread.h"

#include "libyuv/convert.h"
#include "libyuv/video_common.h"
#include "common_video/libyuv/include/webrtc_libyuv.h"

TSharedPtr<FPixelCaptureCapturerNV12ToRHI> FPixelCaptureCapturerNV12ToRHI::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerNV12ToRHI>(new FPixelCaptureCapturerNV12ToRHI(Config));
}

FPixelCaptureCapturerNV12ToRHI::FPixelCaptureCapturerNV12ToRHI(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

FPixelCaptureCapturerNV12ToRHI::~FPixelCaptureCapturerNV12ToRHI()
{
	if (R8Buffer)
	{
		delete[] R8Buffer;
	}
}

void FPixelCaptureCapturerNV12ToRHI::Initialize(int32 InputWidth, int32 InputHeight)
{
	Config.OutputResolution = { InputWidth, InputHeight };

	const uint32_t Size = (3 * Config.OutputResolution.X * Config.OutputResolution.Y) >> 1;
	R8Buffer = new uint8_t[Size];

	FPixelCaptureCapturer::Initialize(InputWidth, InputHeight);
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerNV12ToRHI::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	FRHITextureCreateDesc TextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerNV12ToRHI Texture"), InputWidth, 3 * InputHeight / 2, EPixelFormat::PF_R8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::CPUWritable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::Present)
			.DetermineInititialState();

	if (Config.bIsSRGB)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::SRGB);
	}

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::External);
	}
	else
	{
		TextureDesc.AddFlags(ETextureCreateFlags::Shared);
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	return new FPixelCaptureOutputFrameRHI(RHICmdList.CreateTexture(TextureDesc), { InputWidth, InputHeight, EPixelFormat::PF_NV12 });
}

void FPixelCaptureCapturerNV12ToRHI::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	if (InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_NV12))
	{
		UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

		const FPixelCaptureInputFrameNV12&		SourceFrame = StaticCast<const FPixelCaptureInputFrameNV12&>(InputFrame);
		TSharedPtr<FPixelCaptureBufferNV12>		NV12Buffer = SourceFrame.GetBuffer();
		TSharedPtr<FPixelCaptureOutputFrameRHI> OutputTexture = StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputBuffer);

		checkf(OutputTexture->GetFrameTexture()->GetDesc().Format == EPixelFormat::PF_R8, TEXT("Texture Format is incorrect for NV12 storage"))

			TWeakPtr<FPixelCaptureCapturerNV12ToRHI>
				WeakSelf = StaticCastSharedRef<FPixelCaptureCapturerNV12ToRHI>(AsShared());
		ENQUEUE_RENDER_COMMAND(PixelCaptureNV12ToRHI)([WeakSelf, OutputTexture, NV12Buffer](FRHICommandListImmediate& RHICmdList) {
			if (TSharedPtr<FPixelCaptureCapturerNV12ToRHI> Self = WeakSelf.Pin())
			{
				UE::PixelCapture::MarkGPUWorkStart(OutputTexture);

				// update our staging texture
				FUpdateTextureRegion2D Region(0, 0, 0, 0, Self->Config.OutputResolution.X, Self->Config.OutputResolution.Y * 1.5);
				RHICmdList.UpdateTexture2D(OutputTexture->GetFrameTexture(), 0, Region, Self->Config.OutputResolution.X, NV12Buffer->GetData());

				UE::PixelCapture::MarkGPUWorkEnd(OutputTexture);

				Self->EndProcess(OutputTexture);
				Self->SetIsBusy(false);
			}
		});

		UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);
	}
	else
	{
		unimplemented();
	}
}
