// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerRHI.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureUtils.h"

#include "Async/Async.h"

TSharedPtr<FPixelCaptureCapturerRHI> FPixelCaptureCapturerRHI::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerRHI>(new FPixelCaptureCapturerRHI(Config));
}

FPixelCaptureCapturerRHI::FPixelCaptureCapturerRHI(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
	RHIType = RHIGetInterfaceType();
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerRHI::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	FRHITextureCreateDesc TextureDesc = FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerRHI Texture"), Config.OutputResolution.X, Config.OutputResolution.Y, EPixelFormat::PF_B8G8R8A8);

	if (RHIGetInterfaceType() == ERHIInterfaceType::Metal)
	{
		TextureDesc.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::CPUReadback)
			.SetInitialState(ERHIAccess::CPURead);
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12 || RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::RenderTargetable)
			.SetInitialState(ERHIAccess::Present);
	}

	if (Config.bIsSRGB)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::SRGB);
	}

	TextureDesc.DetermineInititialState();

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::External);
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::Shared);
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	return new FPixelCaptureOutputFrameRHI(RHICmdList.CreateTexture(TextureDesc));
}

void FPixelCaptureCapturerRHI::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
	RHICmdList.EnqueueLambda([this, OutputBuffer](FRHICommandListImmediate&) { UE::PixelCapture::MarkGPUWorkStart(OutputBuffer); });

	const FPixelCaptureInputFrameRHI&		RHISourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);
	TSharedPtr<FPixelCaptureOutputFrameRHI> OutputH264Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputBuffer);
	FGPUFenceRHIRef							Fence = GDynamicRHI->RHICreateGPUFence(TEXT("FPixelCaptureCapturerRHI Fence"));
	CopyTexture(RHICmdList, RHISourceFrame.FrameTexture, OutputH264Buffer->GetFrameTexture(), Fence);

	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	// by adding this shared ref to the async lambda we can ensure that 'this' will not be destroyed
	// until after the rhi thread is done with it, so all the commands will still have valid references.
	TSharedRef<FPixelCaptureCapturerRHI> ThisRHIRef = StaticCastSharedRef<FPixelCaptureCapturerRHI>(AsShared());
	AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [ThisRHIRef, OutputBuffer, Fence]() { ThisRHIRef->CheckComplete(OutputBuffer, Fence); });
}

void FPixelCaptureCapturerRHI::CheckComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer, FGPUFenceRHIRef Fence)
{
	// TODO: We should move to proper event driven fences once they're implemented. Both DX12 and Vulkan APIs support them, they just haven't been added
	// to their respective RHIs. DX11_3 supports it but for compatability reasons we can't upgrade from DX11_2.

	// in lieu of a proper callback we need to capture a thread to poll the fence
	// so we know as quickly as possible when we can readback.

	// sometimes we end up in a deadlock when we loop here polling the fence
	// so instead we check and then submit a new check task.
	if (!Fence->Poll())
	{
		TSharedRef<FPixelCaptureCapturerRHI> ThisRHIRef = StaticCastSharedRef<FPixelCaptureCapturerRHI>(AsShared());
		AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [ThisRHIRef, OutputBuffer, Fence]() { ThisRHIRef->CheckComplete(OutputBuffer, Fence); });
	}
	else
	{
		Fence->Clear();
		OnRHIStageComplete(OutputBuffer);
	}
}

void FPixelCaptureCapturerRHI::OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);

	EndProcess(OutputBuffer);
}
