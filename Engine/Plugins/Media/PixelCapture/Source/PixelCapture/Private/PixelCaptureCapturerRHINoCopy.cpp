// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerRHINoCopy.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureUtils.h"

#include "Async/Async.h"

TSharedPtr<FPixelCaptureCapturerRHINoCopy> FPixelCaptureCapturerRHINoCopy::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerRHINoCopy>(new FPixelCaptureCapturerRHINoCopy(Config));
}

FPixelCaptureCapturerRHINoCopy::FPixelCaptureCapturerRHINoCopy(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerRHINoCopy::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	return new FPixelCaptureOutputFrameRHI(nullptr);
}

void FPixelCaptureCapturerRHINoCopy::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

	const FPixelCaptureInputFrameRHI&		RHISourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);
	TSharedPtr<FPixelCaptureOutputFrameRHI> OutputH264Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputBuffer);
	OutputH264Buffer->SetFrameTexture(RHISourceFrame.FrameTexture);

	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	UE::PixelCapture::MarkGPUWorkStart(OutputBuffer);
	UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);

	EndProcess(OutputBuffer);
}