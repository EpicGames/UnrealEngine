// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerRHIRDG.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureUtils.h"
#include "RenderGraphUtils.h"
#include "Async/Async.h"

TSharedPtr<FPixelCaptureCapturerRHIRDG> FPixelCaptureCapturerRHIRDG::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerRHIRDG>(new FPixelCaptureCapturerRHIRDG(Config));
}

FPixelCaptureCapturerRHIRDG::FPixelCaptureCapturerRHIRDG(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerRHIRDG::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	FRHITextureCreateDesc TextureDesc = FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerRHIRDG Texture"), Config.OutputResolution.X, Config.OutputResolution.Y, EPixelFormat::PF_B8G8R8A8);

	if (RHIGetInterfaceType() == ERHIInterfaceType::Metal)
	{
		TextureDesc.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::CPUReadback)
			.SetInitialState(ERHIAccess::CPURead);
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12 || RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::RenderTargetable)
			.SetInitialState(ERHIAccess::Present);
	}

	if (Config.bIsSRGB)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::SRGB);
	}

	TextureDesc.DetermineInititialState();

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::External);
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::Shared);
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	return new FPixelCaptureOutputFrameRHI(RHICmdList.CreateTexture(TextureDesc));
}

void FPixelCaptureCapturerRHIRDG::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);
	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();

	UE::PixelCapture::MarkGPUWorkStart(OutputBuffer);
	const FPixelCaptureInputFrameRHI&		RHISourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);
	TSharedPtr<FPixelCaptureOutputFrameRHI> OutputH264Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputBuffer);
	CopyTextureRDG(RHICmdList, RHISourceFrame.FrameTexture, OutputH264Buffer->GetFrameTexture());

	UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);
	OnRHIStageComplete(OutputBuffer);
}

void FPixelCaptureCapturerRHIRDG::CheckComplete()
{
}

void FPixelCaptureCapturerRHIRDG::OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	EndProcess(OutputBuffer);
	SetIsBusy(false);
}
