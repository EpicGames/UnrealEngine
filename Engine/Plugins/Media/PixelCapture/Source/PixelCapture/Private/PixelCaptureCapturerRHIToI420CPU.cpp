// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerRHIToI420CPU.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureUtils.h"

#include "Async/Async.h"
#include "Containers/UnrealString.h"

#include "libyuv/convert.h"

TSharedPtr<FPixelCaptureCapturerRHIToI420CPU> FPixelCaptureCapturerRHIToI420CPU::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerRHIToI420CPU>(new FPixelCaptureCapturerRHIToI420CPU(Config));
}

FPixelCaptureCapturerRHIToI420CPU::FPixelCaptureCapturerRHIToI420CPU(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

FPixelCaptureCapturerRHIToI420CPU::~FPixelCaptureCapturerRHIToI420CPU()
{
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerRHIToI420CPU::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	return new FPixelCaptureOutputFrameI420(MakeShared<FPixelCaptureBufferI420>(Config.OutputResolution.X, Config.OutputResolution.Y));
}

#if 1
// Old code path that uses a FRHIGPUTextureReadback. This seems to induce extra latency but does not introduce screen tearing
void FPixelCaptureCapturerRHIToI420CPU::Initialize(int32 InputWidth, int32 InputHeight)
{
	FRHITextureCreateDesc TextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerRHIToI420CPU StagingTexture"), Config.OutputResolution.X, Config.OutputResolution.Y, EPixelFormat::PF_B8G8R8A8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::RenderTargetable)
			.SetInitialState(ERHIAccess::CopySrc)
			.DetermineInititialState();

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		TextureDesc.AddFlags(ETextureCreateFlags::External);
	}
	else
	{
		TextureDesc.AddFlags(ETextureCreateFlags::Shared);
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	StagingTexture = RHICmdList.CreateTexture(TextureDesc);

	TextureReader = MakeShared<FRHIGPUTextureReadback>(TEXT("FPixelCaptureCapturerRHIToI420CPUReadback"));

	FPixelCaptureCapturer::Initialize(InputWidth, InputHeight);
}

void FPixelCaptureCapturerRHIToI420CPU::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

	const FPixelCaptureInputFrameRHI& RHISourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);
	FTextureRHIRef					  SourceTexture = RHISourceFrame.FrameTexture;

	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();

	RHICmdList.EnqueueLambda([this, OutputBuffer](FRHICommandListImmediate&) { UE::PixelCapture::MarkGPUWorkStart(OutputBuffer); });

	RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc));
	RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::CopySrc, ERHIAccess::CopyDest));
	CopyTexture(RHICmdList, SourceTexture, StagingTexture, nullptr);
	RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	TextureReader->EnqueueCopy(RHICmdList, StagingTexture, FIntVector(0, 0, 0), 0, FIntVector(StagingTexture->GetSizeXY().X, StagingTexture->GetSizeXY().Y, 0));

	// by adding this shared ref to the rhi lambda we can ensure that 'this' will not be destroyed
	// until after the rhi thread is done with it, so all the commands will still have valid references.
	TSharedRef<FPixelCaptureCapturerRHIToI420CPU> ThisRHIRef = StaticCastSharedRef<FPixelCaptureCapturerRHIToI420CPU>(AsShared());
	RHICmdList.EnqueueLambda([ThisRHIRef, OutputBuffer](FRHICommandListImmediate&) {
		ThisRHIRef->CheckComplete(OutputBuffer);
	});
}

void FPixelCaptureCapturerRHIToI420CPU::CheckComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	if (!TextureReader->IsReady())
	{
		TSharedRef<FPixelCaptureCapturerRHIToI420CPU> ThisRHIRef = StaticCastSharedRef<FPixelCaptureCapturerRHIToI420CPU>(AsShared());
		AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [ThisRHIRef, OutputBuffer]() { ThisRHIRef->CheckComplete(OutputBuffer); });
	}
	else
	{
		TSharedRef<FPixelCaptureCapturerRHIToI420CPU> ThisRHIRef = StaticCastSharedRef<FPixelCaptureCapturerRHIToI420CPU>(AsShared());
		AsyncTask(ENamedThreads::ActualRenderingThread, [ThisRHIRef, OutputBuffer]() { ThisRHIRef->OnRHIStageComplete(OutputBuffer); });
	}
}

void FPixelCaptureCapturerRHIToI420CPU::OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);

	int32 OutRowPitchInPixels = 0;
	void* PixelData = TextureReader->Lock(OutRowPitchInPixels);

	TSharedPtr<FPixelCaptureOutputFrameI420> OutputI420Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameI420>(OutputBuffer);
	TSharedPtr<FPixelCaptureBufferI420>		 I420Buffer = OutputI420Buffer->GetI420Buffer();
	libyuv::ARGBToI420(
		static_cast<uint8*>(PixelData),
		OutRowPitchInPixels * 4,
		I420Buffer->GetMutableDataY(),
		I420Buffer->GetStrideY(),
		I420Buffer->GetMutableDataU(),
		I420Buffer->GetStrideUV(),
		I420Buffer->GetMutableDataV(),
		I420Buffer->GetStrideUV(),
		I420Buffer->GetWidth(),
		I420Buffer->GetHeight());

	TextureReader->Unlock();

	EndProcess(OutputBuffer);
	SetIsBusy(false);
}

// TODO (Eden.Harris) This has less latency but introduces screen tearing so is disabled. RTCP-7778 to fix the screen tearing without effecting latency.
#else

void FPixelCaptureCapturerRHIToI420CPU::Initialize(int32 InputWidth, int32 InputHeight)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	FRHITextureCreateDesc StagingTextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerRHIToI420CPU StagingTexture"), Config.OutputResolution.X, Config.OutputResolution.Y, EPixelFormat::PF_B8G8R8A8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::RenderTargetable)
			.SetInitialState(ERHIAccess::CopySrc)
			.DetermineInititialState();

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		StagingTextureDesc.AddFlags(ETextureCreateFlags::External);
	}
	else
	{
		StagingTextureDesc.AddFlags(ETextureCreateFlags::Shared);
	}

	StagingTexture = RHICmdList.CreateTexture(StagingTextureDesc);

	FRHITextureCreateDesc ReadbackTextureDesc = FRHITextureCreateDesc::Create2D(TEXT("FPixelCaptureCapturerRHIToI420CPU ReadbackTexture"), Config.OutputResolution.X, Config.OutputResolution.Y, EPixelFormat::PF_B8G8R8A8)
													.SetClearValue(FClearValueBinding::None)
													.SetFlags(ETextureCreateFlags::CPUReadback | ETextureCreateFlags::HideInVisualizeTexture)
													.SetInitialState(ERHIAccess::CopyDest)
													.DetermineInititialState();

	ReadbackTexture = RHICmdList.CreateTexture(ReadbackTextureDesc);

	FPixelCaptureCapturer::Initialize(InputWidth, InputHeight);
}

void FPixelCaptureCapturerRHIToI420CPU::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));

	const FPixelCaptureInputFrameRHI& RHISourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);
	FTextureRHIRef					  SourceTexture = RHISourceFrame.FrameTexture;

	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
	RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc));
	RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::CopySrc, ERHIAccess::CopyDest));
	CopyTexture(RHICmdList, SourceTexture, StagingTexture, nullptr);
	RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));
	// Format and extent match so this is just a simple copy
	CopyTexture(RHICmdList, StagingTexture, ReadbackTexture, nullptr);

	void* ReadbackPointer = nullptr;
	int32 ReadbackWidth, ReadbackHeight;
	// Nullptr fence forces a flush
	RHICmdList.MapStagingSurface(ReadbackTexture, nullptr, ReadbackPointer, ReadbackWidth, ReadbackHeight);

	TSharedPtr<FPixelCaptureOutputFrameI420> OutputI420Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameI420>(OutputBuffer);
	TSharedPtr<FPixelCaptureBufferI420>		 I420Buffer = OutputI420Buffer->GetI420Buffer();
	libyuv::ARGBToI420(
		static_cast<uint8*>(ReadbackPointer),
		ReadbackWidth * 4,
		I420Buffer->GetMutableDataY(),
		I420Buffer->GetStrideY(),
		I420Buffer->GetMutableDataU(),
		I420Buffer->GetStrideUV(),
		I420Buffer->GetMutableDataV(),
		I420Buffer->GetStrideUV(),
		I420Buffer->GetWidth(),
		I420Buffer->GetHeight());

	RHICmdList.UnmapStagingSurface(ReadbackTexture);

	EndProcess(OutputBuffer);
	SetIsBusy(false);
}
#endif