// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureCapturerRHIToI420Compute.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureBufferFormat.h"

#include "RGBToYUVShader.h"

namespace
{
	inline void MemCpyStride(void* Dest, const void* Src, size_t DestStride, size_t SrcStride, size_t Height)
	{
		char*		DestPtr = static_cast<char*>(Dest);
		const char* SrcPtr = static_cast<const char*>(Src);
		size_t		Row = Height;
		while (Row--)
		{
			FMemory::Memcpy(DestPtr + DestStride * Row, SrcPtr + SrcStride * Row, DestStride);
		}
	}
} // namespace

TSharedPtr<FPixelCaptureCapturerRHIToI420Compute> FPixelCaptureCapturerRHIToI420Compute::Create(FPixelCaptureCapturerConfig Config)
{
	return TSharedPtr<FPixelCaptureCapturerRHIToI420Compute>(new FPixelCaptureCapturerRHIToI420Compute(Config));
}

FPixelCaptureCapturerRHIToI420Compute::FPixelCaptureCapturerRHIToI420Compute(FPixelCaptureCapturerConfig& Config)
	: FPixelCaptureCapturer(Config)
{
}

FPixelCaptureCapturerRHIToI420Compute ::~FPixelCaptureCapturerRHIToI420Compute()
{
	CleanUp();
}

void FPixelCaptureCapturerRHIToI420Compute::Initialize(int32 InputWidth, int32 InputHeight)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	PlaneYDimensions = { Config.OutputResolution.X, Config.OutputResolution.Y };
	PlaneUVDimensions = { (Config.OutputResolution.X + 1) / 2, (Config.OutputResolution.Y + 1) / 2 }; // UV is halved and rounded up

	FRHITextureCreateDesc TextureDescY =
		FRHITextureCreateDesc::Create2D(TEXT("Compute YUV Target"), PlaneYDimensions.X, PlaneYDimensions.Y, EPixelFormat::PF_R8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::UAV)
			.SetInitialState(ERHIAccess::UAVCompute)
			.DetermineInititialState();

	FRHITextureCreateDesc TextureDescUV =
		FRHITextureCreateDesc::Create2D(TEXT("Compute YUV Target"), PlaneUVDimensions.X, PlaneUVDimensions.Y, EPixelFormat::PF_R8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::UAV)
			.SetInitialState(ERHIAccess::UAVCompute)
			.DetermineInititialState();

	TextureY = RHICmdList.CreateTexture(TextureDescY);
	TextureU = RHICmdList.CreateTexture(TextureDescUV);
	TextureV = RHICmdList.CreateTexture(TextureDescUV);

	FRHITextureCreateDesc StagingDescY =
		FRHITextureCreateDesc::Create2D(TEXT("YUV Output CPU Texture"), PlaneYDimensions.X, PlaneYDimensions.Y, EPixelFormat::PF_R8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::CPUReadback)
			.SetInitialState(ERHIAccess::Unknown)
			.DetermineInititialState();

	FRHITextureCreateDesc StagingDescUV =
		FRHITextureCreateDesc::Create2D(TEXT("YUV Output CPU Texture"), PlaneUVDimensions.X, PlaneUVDimensions.Y, EPixelFormat::PF_R8)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(ETextureCreateFlags::CPUReadback)
			.SetInitialState(ERHIAccess::Unknown)
			.DetermineInititialState();

	StagingTextureY = RHICmdList.CreateTexture(StagingDescY);
	StagingTextureU = RHICmdList.CreateTexture(StagingDescUV);
	StagingTextureV = RHICmdList.CreateTexture(StagingDescUV);

	TextureYUAV = RHICmdList.CreateUnorderedAccessView(TextureY, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(TextureY));
	TextureUUAV = RHICmdList.CreateUnorderedAccessView(TextureU, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(TextureU));
	TextureVUAV = RHICmdList.CreateUnorderedAccessView(TextureV, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(TextureV));

	int32 OutWidth, OutHeight;
	RHICmdList.MapStagingSurface(StagingTextureY, nullptr, MappedY, OutWidth, OutHeight);
	YStride = OutWidth;
	RHICmdList.MapStagingSurface(StagingTextureU, nullptr, MappedU, OutWidth, OutHeight);
	UStride = OutWidth;
	RHICmdList.MapStagingSurface(StagingTextureV, nullptr, MappedV, OutWidth, OutHeight);
	VStride = OutWidth;

	FPixelCaptureCapturer::Initialize(InputWidth, InputHeight);
}

IPixelCaptureOutputFrame* FPixelCaptureCapturerRHIToI420Compute::CreateOutputBuffer(int32 InputWidth, int32 InputHeight)
{
	return new FPixelCaptureOutputFrameI420(MakeShared<FPixelCaptureBufferI420>(Config.OutputResolution.X, Config.OutputResolution.Y));
}

void FPixelCaptureCapturerRHIToI420Compute::BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	SetIsBusy(true);

	checkf(InputFrame.GetType() == StaticCast<int32>(PixelCaptureBufferFormat::FORMAT_RHI), TEXT("Incorrect source frame coming into frame capture process."));

	UE::PixelCapture::MarkCPUWorkStart(OutputBuffer);

	const FPixelCaptureInputFrameRHI& RHISourceFrame = StaticCast<const FPixelCaptureInputFrameRHI&>(InputFrame);
	FTextureRHIRef					  SourceTexture = RHISourceFrame.FrameTexture;

	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();

	RHICmdList.EnqueueLambda([this, OutputBuffer](FRHICommandListImmediate&) { UE::PixelCapture::MarkGPUWorkStart(OutputBuffer); });

	FRGBToYUVShaderParameters ShaderParameters;
	ShaderParameters.SourceTexture = SourceTexture;
	ShaderParameters.DestPlaneYDimensions = PlaneYDimensions;
	ShaderParameters.DestPlaneUVDimensions = PlaneUVDimensions;
	ShaderParameters.DestPlaneY = TextureYUAV;
	ShaderParameters.DestPlaneU = TextureUUAV;
	ShaderParameters.DestPlaneV = TextureVUAV;
	FRGBToYUVShader::Dispatch(RHICmdList, ShaderParameters);

	RHICmdList.CopyTexture(TextureY, StagingTextureY, {});
	RHICmdList.CopyTexture(TextureU, StagingTextureU, {});
	RHICmdList.CopyTexture(TextureV, StagingTextureV, {});

	UE::PixelCapture::MarkCPUWorkEnd(OutputBuffer);

	// by adding this shared ref to the rhi lambda we can ensure that 'this' will not be destroyed
	// until after the rhi thread is done with it, so all the commands will still have valid references.
	TSharedRef<FPixelCaptureCapturerRHIToI420Compute> ThisRHIRef = StaticCastSharedRef<FPixelCaptureCapturerRHIToI420Compute>(AsShared());
	RHICmdList.EnqueueLambda([ThisRHIRef, OutputBuffer](FRHICommandListImmediate&) { ThisRHIRef->OnRHIStageComplete(OutputBuffer); });
}

void FPixelCaptureCapturerRHIToI420Compute::OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
{
	UE::PixelCapture::MarkGPUWorkEnd(OutputBuffer);

	TSharedPtr<FPixelCaptureOutputFrameI420> OutputI420Buffer = StaticCastSharedPtr<FPixelCaptureOutputFrameI420>(OutputBuffer);
	TSharedPtr<FPixelCaptureBufferI420>		 I420Buffer = OutputI420Buffer->GetI420Buffer();

	MemCpyStride(I420Buffer->GetMutableDataY(), MappedY, I420Buffer->GetStrideY(), YStride, PlaneYDimensions.Y);
	MemCpyStride(I420Buffer->GetMutableDataU(), MappedU, I420Buffer->GetStrideUV(), UStride, PlaneUVDimensions.Y);
	MemCpyStride(I420Buffer->GetMutableDataV(), MappedV, I420Buffer->GetStrideUV(), VStride, PlaneUVDimensions.Y);

	EndProcess(OutputBuffer);
	SetIsBusy(false);
}

void FPixelCaptureCapturerRHIToI420Compute::CleanUp()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListExecutor::GetImmediateCommandList();
	RHICmdList.UnmapStagingSurface(StagingTextureY);
	RHICmdList.UnmapStagingSurface(StagingTextureU);
	RHICmdList.UnmapStagingSurface(StagingTextureV);

	MappedY = nullptr;
	MappedU = nullptr;
	MappedV = nullptr;
}
