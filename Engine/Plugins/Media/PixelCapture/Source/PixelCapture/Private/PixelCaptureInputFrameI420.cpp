// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureInputFrameI420.h"
#include "PixelCaptureBufferFormat.h"

FPixelCaptureInputFrameI420::FPixelCaptureInputFrameI420(TSharedPtr<FPixelCaptureBufferI420> Buffer, TSharedPtr<FPixelCaptureUserData> UserData)
	: IPixelCaptureInputFrame(UserData)
	, I420Buffer(Buffer)
{
}

int32 FPixelCaptureInputFrameI420::GetType() const
{
	return I420Buffer->GetFormat();
}

int32 FPixelCaptureInputFrameI420::GetWidth() const
{
	return I420Buffer->GetWidth();
}

int32 FPixelCaptureInputFrameI420::GetHeight() const
{
	return I420Buffer->GetHeight();
}

TSharedPtr<FPixelCaptureBufferI420> FPixelCaptureInputFrameI420::GetBuffer() const
{
	return I420Buffer;
}
