// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureInputFrameNV12.h"
#include "PixelCaptureBufferFormat.h"

FPixelCaptureInputFrameNV12::FPixelCaptureInputFrameNV12(TSharedPtr<FPixelCaptureBufferNV12> Buffer, TSharedPtr<FPixelCaptureUserData> UserData)
	: IPixelCaptureInputFrame(UserData)
	, NV12Buffer(Buffer)
{
}

int32 FPixelCaptureInputFrameNV12::GetType() const
{
	return NV12Buffer->GetFormat();
}

int32 FPixelCaptureInputFrameNV12::GetWidth() const
{
	return NV12Buffer->GetWidth();
}

int32 FPixelCaptureInputFrameNV12::GetHeight() const
{
	return NV12Buffer->GetHeight();
}

TSharedPtr<FPixelCaptureBufferNV12> FPixelCaptureInputFrameNV12::GetBuffer() const
{
	return NV12Buffer;
}
