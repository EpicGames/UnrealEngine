// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureBufferFormat.h"

FPixelCaptureInputFrameRHI::FPixelCaptureInputFrameRHI(FTextureRHIRef InFrameTexture, TSharedPtr<FPixelCaptureUserData> UserData)
	: IPixelCaptureInputFrame(UserData)
	, FrameTexture(InFrameTexture)
{
}

int32 FPixelCaptureInputFrameRHI::GetType() const
{
	return static_cast<int32>(PixelCaptureBufferFormat::FORMAT_RHI);
}

int32 FPixelCaptureInputFrameRHI::GetWidth() const
{
	return FrameTexture->GetDesc().Extent.X;
}

int32 FPixelCaptureInputFrameRHI::GetHeight() const
{
	return FrameTexture->GetDesc().Extent.Y;
}