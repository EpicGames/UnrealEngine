// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"
#include "PixelCapturePrivate.h"

DEFINE_LOG_CATEGORY(LogPixelCapture);

class FPixelCaptureModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FPixelCaptureModule, PixelCapture)
