// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelCapture, Log, All);