// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaShaders.h"
#include "Modules/ModuleManager.h"
#include "RHI.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "RenderGraphUtils.h"

/*
    * Copy from one texture to another.
    * Assumes SourceTexture is in ERHIAccess::CopySrc and DestTexture is in ERHIAccess::CopyDest
    * Fence can be nullptr if no fence is to be used.
    */
inline void CopyTexture(FRHICommandList& RHICmdList, FTextureRHIRef SourceTexture, FTextureRHIRef DestTexture, FRHIGPUFence* Fence)
{
    if (SourceTexture->GetDesc().Format == DestTexture->GetDesc().Format
        && SourceTexture->GetDesc().Extent.X == DestTexture->GetDesc().Extent.X
        && SourceTexture->GetDesc().Extent.Y == DestTexture->GetDesc().Extent.Y)
    {

        RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc));
        RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest));

        // source and dest are the same. simple copy
        RHICmdList.CopyTexture(SourceTexture, DestTexture, {});
    }
    else
    {
        IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>("Renderer");

        RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
        RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

        // source and destination are different. rendered copy
        FRHIRenderPassInfo RPInfo(DestTexture, ERenderTargetActions::Load_Store);
        RHICmdList.BeginRenderPass(RPInfo, TEXT("PixelCapture::CopyTexture"));
        {
            FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
            TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
            TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

            RHICmdList.SetViewport(0, 0, 0.0f, DestTexture->GetDesc().Extent.X, DestTexture->GetDesc().Extent.Y, 1.0f);

            FGraphicsPipelineStateInitializer GraphicsPSOInit;
            RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
            GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
            GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
            GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
            GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
            GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
            GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
            GraphicsPSOInit.PrimitiveType = PT_TriangleList;
            SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), SourceTexture);

            FIntPoint TargetBufferSize(DestTexture->GetDesc().Extent.X, DestTexture->GetDesc().Extent.Y);
            RendererModule->DrawRectangle(RHICmdList, 0, 0, // Dest X, Y
                DestTexture->GetDesc().Extent.X,			// Dest Width
                DestTexture->GetDesc().Extent.Y,			// Dest Height
                0, 0,										// Source U, V
                1, 1,										// Source USize, VSize
                TargetBufferSize,							// Target buffer size
                FIntPoint(1, 1),							// Source texture size
                VertexShader, EDRF_Default);
        }

        RHICmdList.EndRenderPass();

        RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
        RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::RTV, ERHIAccess::CopyDest));
    }

    if (Fence != nullptr)
    {
        RHICmdList.WriteGPUFence(Fence);
    }
}

inline void CopyTextureRDG(FRHICommandListImmediate& RHICmdList, FTextureRHIRef SourceTexture, FTextureRHIRef DestTexture)
{
	FRDGBuilder GraphBuilder(RHICmdList);

	// Register an external RDG rexture from the source texture
	FRDGTexture* InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTexture, TEXT("PixelCaptureCopySourceTexture")));

	// Register an external RDG texture from the output buffer
	FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestTexture, TEXT("PixelCaptureCopyDestTexture")));

	if (InputTexture->Desc.Format == OutputTexture->Desc.Format &&
        InputTexture->Desc.Extent.X == OutputTexture->Desc.Extent.X &&
        InputTexture->Desc.Extent.Y == OutputTexture->Desc.Extent.Y)
    {
		// The formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			InputTexture,
			OutputTexture,
			FRDGDrawTextureInfo()
		);
	}
	else
	{
		// TODO (matthew.cotton) A lot of this was copied from AddScreenPass and altered for our use.
		// This was altered because the FViewInfo we create here on the stack is added to the RHI
		// queue and is accessed possibly after this stack frame was released. Originally we tried
		// just changing it to static so we could reuse it, but the object does not allow copying.
		// So the current implementation creates the View on the heap using a shared ptr that is then
		// captured in a lambda for the RHI queue. This allows us to update the view on calls to this
		// function and not worry about it's lifetime.
		// The todo is here because there has to be a better way to achieve what we want without
		// all of this song and dance.

#if PLATFORM_APPLE
		// Create a staging texture that is the same size and format as the final.
		FRDGTextureRef StagingTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y), OutputTexture->Desc.Format, OutputTexture->Desc.ClearValue, ETextureCreateFlags::RenderTargetable), TEXT("PixelCaptureCopy Staging"));
		FScreenPassTextureViewport StagingViewport(StagingTexture);
#endif

		// The formats or size differ to pixel shader stuff
		//Configure source/output viewport to get the right UV scaling from source texture to output texture
		FScreenPassTextureViewport InputViewport(InputTexture);
		FScreenPassTextureViewport OutputViewport(OutputTexture);

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		int32 ConversionOperation = 0; // None
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(ConversionOperation);

		// Rectangle area to use from source
		const FIntRect ViewRect(FIntPoint(0, 0), InputTexture->Desc.Extent);

		TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
		FModifyAlphaSwizzleRgbaPS::FParameters* PixelShaderParameters = PixelShader->AllocateAndSetParameters(
			GraphBuilder,
			InputTexture,
#if PLATFORM_APPLE
			StagingTexture
#else
			OutputTexture
#endif
			);
		
		FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("PixelCapturerSwizzle"),
			FScreenPassViewInfo(),
#if PLATFORM_APPLE
			StagingViewport,
#else
			OutputViewport,
#endif
			
			InputViewport,
			VertexShader,
			PixelShader,
			PixelShaderParameters);

#if PLATFORM_APPLE
		// Now we can be certain the formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			StagingTexture,
			OutputTexture,
			FRDGDrawTextureInfo()
		);
#endif
	}

	GraphBuilder.Execute();
}
