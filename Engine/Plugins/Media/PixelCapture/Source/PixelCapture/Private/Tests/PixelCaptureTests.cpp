// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureCapturer.h"
#include "PixelCaptureCapturerLayered.h"
#include "PixelCaptureCapturerMultiFormat.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	enum class EMockFormats : int32
	{
		Input = PixelCaptureBufferFormat::FORMAT_USER,
		Output1,
		Output2,
		Output3,
	};

	struct FMockInput : public IPixelCaptureInputFrame
	{
		int32 Width;
		int32 Height;
		int32 MagicData;

		FMockInput(int32 InWidth, int32 InHeight, int32 InMagicData)
			: Width(InWidth)
			, Height(InHeight)
			, MagicData(InMagicData)
		{
		}

		virtual int32 GetType() const override { return StaticCast<int32>(EMockFormats::Input); }
		virtual int32 GetWidth() const override { return Width; }
		virtual int32 GetHeight() const override { return Height; }
	};

	struct FMockOutput : public IPixelCaptureOutputFrame
	{
		int32 Width = 0;
		int32 Height = 0;
		int32 MagicData = 0;

		virtual int32 GetWidth() const override { return Width; }
		virtual int32 GetHeight() const override { return Height; }
	};

	struct FMockCapturer : public FPixelCaptureCapturer
	{
		int32	  Format;
		FIntPoint OutputResolution;
		bool	  AutoComplete;

		FMockCapturer(int32 InFormat, FIntPoint InOutputResolution, bool InAutoComplete)
			: Format(InFormat)
			, OutputResolution(InOutputResolution)
			, AutoComplete(InAutoComplete)
		{
		}

		virtual FString GetCapturerName() const override { return "Mock Capturer"; }

		virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override
		{
			return new FMockOutput();
		}

		virtual void BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override
		{
			SetIsBusy(true);

			check(InputFrame.GetType() == StaticCast<int32>(EMockFormats::Input));
			const FMockInput&		Input = StaticCast<const FMockInput&>(InputFrame);
			TSharedPtr<FMockOutput> Output = StaticCastSharedPtr<FMockOutput>(OutputBuffer);
			Output->Width = OutputResolution.X;
			Output->Height = OutputResolution.Y;
			Output->MagicData = Input.MagicData;

			if (AutoComplete)
			{
				EndProcess(OutputBuffer);
				SetIsBusy(false);
			}
			else
			{
				StoredOutputBuffer = OutputBuffer;
			}
		}

		void MockProcessComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer)
		{
			EndProcess(OutputBuffer);
			SetIsBusy(false);
		}

		TSharedPtr<IPixelCaptureOutputFrame> StoredOutputBuffer;
	};

	struct FMockCaptureSource : public IPixelCaptureCapturerSource
	{
		TAtomic<int>			  FrameCompleteCount = 0;
		TMap<FMockCapturer*, int> LayerCompleteCounts;
		FCriticalSection		  CountSection;

		virtual ~FMockCaptureSource() = default;

		virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint OutputResolution) override
		{
			checkf(FinalFormat == StaticCast<int32>(EMockFormats::Output1)
					|| FinalFormat == StaticCast<int32>(EMockFormats::Output2)
					|| FinalFormat == StaticCast<int32>(EMockFormats::Output3),
				TEXT("Unknown destination format."));
			TSharedPtr<FMockCapturer> NewCapturer = MakeShared<FMockCapturer>(FinalFormat, OutputResolution, true);
			{
				FScopeLock Lock(&CountSection);
				LayerCompleteCounts.Add(NewCapturer.Get(), 0);
			}
			NewCapturer->OnComplete.AddRaw(this, &FMockCaptureSource::OnLayerComplete, NewCapturer.Get());
			return NewCapturer;
		}

		void OnLayerComplete(FMockCapturer* Capturer)
		{
			FScopeLock Lock(&CountSection);
			LayerCompleteCounts[Capturer]++;
		}
	};
} // namespace

namespace UE::PixelCapture
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(MultiCaptureTest, "System.Plugins.PixelCapture.MultiCaptureTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool MultiCaptureTest::RunTest(const FString& Parameters)
	{
		const TArray<FIntPoint> Resolutions = { { 1024, 768 }, { 512, 384 }, { 256, 192 } };

		// NOTE: resolution changes with the same capturer are not supported and the capturer will assert.
		// ideally we would make sure that assert gets fired but AddExpectedError only works on error logs
		const int32 InputWidth = 1024;
		const int32 InputHeight = 768;
		const int32 InputMagicData = 0xDEADBEEF;

		// basic capture behaviour
		{
			FMockCapturer Capturer(StaticCast<int32>(EMockFormats::Output1), 1.0f, false);
			TestTrue("Capturer starts uninitialized", Capturer.IsInitialized() == false);
			TestTrue("Capturer starts not busy", Capturer.IsBusy() == false);
			TestTrue("Capturer starts without output", Capturer.HasOutput() == false);
			TestTrue("Capturer returns null if no input present", Capturer.ReadOutput() == nullptr);
			Capturer.Capture(FMockInput(InputWidth, InputHeight, InputMagicData));
			TestTrue("Capturer initialized after frame input", Capturer.IsInitialized() == true);
			TestTrue("Capturer busy after frame input", Capturer.IsBusy() == true);
			TestTrue("Capturer has no output before completing", Capturer.HasOutput() == false);
			TestTrue("Capturer still returns null after input but not complete", Capturer.ReadOutput() == nullptr);
			Capturer.MockProcessComplete(Capturer.StoredOutputBuffer);
			TestTrue("Capturer still initialized after process complete", Capturer.IsInitialized() == true);
			TestTrue("Capturer no longer busy after process complete", Capturer.IsBusy() == false);
			TestTrue("Capturer has output after completing", Capturer.HasOutput() == true);
			TestTrue("Capturer returns output after process complete", Capturer.ReadOutput() != nullptr);
		}

		FMockCaptureSource CaptureSource;

		// testing simple layered capturer with resolutions known at construction
		{
			CaptureSource.FrameCompleteCount = 0;
			CaptureSource.LayerCompleteCounts.Empty();

			TSharedPtr<FPixelCaptureCapturerLayered> LayeredCapturer = FPixelCaptureCapturerLayered::Create(&CaptureSource, StaticCast<int32>(EMockFormats::Output1), Resolutions);
			LayeredCapturer->OnComplete.AddLambda([&]() { CaptureSource.FrameCompleteCount++; });
			LayeredCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData));
			TestTrue("Frame complete callback called once.", CaptureSource.FrameCompleteCount == 3);
			for (TMap<FMockCapturer*, int>::ElementType& CaptureComplete : CaptureSource.LayerCompleteCounts)
			{
				TestTrue(FString::Printf(TEXT("Layer %p called complete callback once."), CaptureComplete.Key), CaptureComplete.Value == 1);
			}
			for (int i = 0; i < Resolutions.Num(); ++i)
			{
				TSharedPtr<FMockOutput> Layer = StaticCastSharedPtr<FMockOutput>(LayeredCapturer->ReadOutput(Resolutions[i]));
				TestTrue(FString::Printf(TEXT("Layer %d has output with correct values."), i),
					Layer && Layer->Width == Resolutions[i].X && Layer->Height == Resolutions[i].Y && Layer->MagicData == InputMagicData);
			}
		}

		// testing simple layered capturer with resolutions NOT known during construction
		{
			CaptureSource.FrameCompleteCount = 0;
			CaptureSource.LayerCompleteCounts.Empty();

			TSharedPtr<FPixelCaptureCapturerLayered> LayeredCapturer = FPixelCaptureCapturerLayered::Create(&CaptureSource, StaticCast<int32>(EMockFormats::Output1));
			LayeredCapturer->OnComplete.AddLambda([&]() { CaptureSource.FrameCompleteCount++; });
			LayeredCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData));
			TestTrue("Frame complete callback not called as there's no known output resolutions.", CaptureSource.FrameCompleteCount == 0);
			for (int i = 0; i < Resolutions.Num(); ++i)
			{
				// Read output will implicitly create a capturer for this resolution as it hasn't seen it before
				TSharedPtr<FMockOutput> Layer = StaticCastSharedPtr<FMockOutput>(LayeredCapturer->ReadOutput(Resolutions[i]));
				TestTrue(FString::Printf(TEXT("Layer %d has no output"), i), !Layer.IsValid());
			}

			LayeredCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData));
			TestTrue("Frame complete callback called once per output resolution.", CaptureSource.FrameCompleteCount == 3);

			for (TMap<FMockCapturer*, int>::ElementType& CaptureComplete : CaptureSource.LayerCompleteCounts)
			{
				TestTrue(FString::Printf(TEXT("Layer %p called complete callback once."), CaptureComplete.Key), CaptureComplete.Value == 1);
			}
			for (int i = 0; i < Resolutions.Num(); ++i)
			{
				TSharedPtr<FMockOutput> Layer = StaticCastSharedPtr<FMockOutput>(LayeredCapturer->ReadOutput(Resolutions[i]));
				TestTrue(FString::Printf(TEXT("Layer %d has output with correct values."), i),
					Layer && Layer->Width == Resolutions[i].X && Layer->Height == Resolutions[i].Y && Layer->MagicData == InputMagicData);
			}
		}

		TSharedPtr<FPixelCaptureCapturerMultiFormat> FormatCapturer = FPixelCaptureCapturerMultiFormat::Create(&CaptureSource, Resolutions);

		// testing mutli format capture
		{
			const int32 TestFormat = StaticCast<int>(EMockFormats::Output1);

			// test that WaitForFormat will time out
			{
				const double PreWaitSeconds = FPlatformTime::Seconds();
				FormatCapturer->WaitForFormat(TestFormat, Resolutions[0], 500);
				const double DeltaSeconds = FPlatformTime::Seconds() - PreWaitSeconds;
				TestTrue("WaitForFormat times out in a reasonable time.", FMath::Abs(DeltaSeconds) < 600);
			}

			// Test output after implicit format add from WaitForFormat
			{
				FormatCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData));
				TestTrue("MultiFormat Capturer reports correct layer count after input.", FormatCapturer->GetNumLayers() == Resolutions.Num());
				for (int i = 0; i < Resolutions.Num(); ++i)
				{
					TSharedPtr<FMockOutput> Layer = StaticCastSharedPtr<FMockOutput>(FormatCapturer->RequestFormat(TestFormat, Resolutions[i]));
					TestTrue(FString::Printf(TEXT("Format %d Layer %d has output with correct values."), TestFormat, i),
						Layer && Layer->Width == Resolutions[i].X && Layer->Height == Resolutions[i].Y && Layer->MagicData == InputMagicData);
				}
			}

			TestTrue("WaitForFormat returns not null after frame input.", FormatCapturer->WaitForFormat(TestFormat, Resolutions[0], 10) != nullptr);
		}

		// test explicit format add
		{
			const int32 TestFormat = StaticCast<int>(EMockFormats::Output2);
			FormatCapturer->AddOutputFormat(TestFormat);
			FormatCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData));
			for (int i = 0; i < Resolutions.Num(); ++i)
			{
				TSharedPtr<FMockOutput> Layer = StaticCastSharedPtr<FMockOutput>(FormatCapturer->RequestFormat(TestFormat, Resolutions[i]));
				TestTrue(FString::Printf(TEXT("Format %d Layer %d has output with correct values."), TestFormat, i),
					Layer && Layer->Width == Resolutions[i].X && Layer->Height == Resolutions[i].Y && Layer->MagicData == InputMagicData);
			}
		}

		// make sure all formats get new frames
		{
			const int32 InputMagicData2 = 0xBAADF00D;
			FormatCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData2));

			auto CheckFormats = { StaticCast<int>(EMockFormats::Output1), StaticCast<int>(EMockFormats::Output2) };
			for (auto& TestFormat : CheckFormats)
			{
				for (int i = 0; i < Resolutions.Num(); ++i)
				{
					TSharedPtr<FMockOutput> Layer = StaticCastSharedPtr<FMockOutput>(FormatCapturer->RequestFormat(TestFormat, Resolutions[i]));
					TestTrue(FString::Printf(TEXT("Format %d Layer %d has output with correct values."), TestFormat, i),
						Layer && Layer->Width == Resolutions[i].X && Layer->Height == Resolutions[i].Y && Layer->MagicData == InputMagicData2);
				}
			}
		}

		// test the callbacks
		{
			CaptureSource.FrameCompleteCount = 0;
			CaptureSource.LayerCompleteCounts.Empty();

			// new capturer to force clear resolutions
			FormatCapturer = FPixelCaptureCapturerMultiFormat::Create(&CaptureSource, Resolutions);
			FormatCapturer->OnComplete.AddLambda([&]() { CaptureSource.FrameCompleteCount++; });

			FormatCapturer->AddOutputFormat(StaticCast<int>(EMockFormats::Output1));
			FormatCapturer->AddOutputFormat(StaticCast<int>(EMockFormats::Output2));
			FormatCapturer->Capture(FMockInput(InputWidth, InputHeight, InputMagicData));

			TestTrue("MultiFormat Frame complete callback called once.", CaptureSource.FrameCompleteCount == 6);
			for (TMap<FMockCapturer*, int>::ElementType& CaptureComplete : CaptureSource.LayerCompleteCounts)
			{
				TestTrue(FString::Printf(TEXT("Format %d Layer %p called complete callback once."), CaptureComplete.Key->Format, CaptureComplete.Key), CaptureComplete.Value == 1);
			}
		}

		return true;
	}
} // namespace UE::PixelCapture

#endif // WITH_DEV_AUTOMATION_TESTS
