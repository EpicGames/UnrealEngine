// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelCaptureBufferFormat.h"

/**
 * Buffer Template for defining frame data.
 */
class IPixelCaptureBuffer
{
public:
	/**
	 * Get the width of the frame
	 */
	virtual int32 GetWidth() const = 0;

	/**
	 * Get the height of the frame
	 */
	virtual int32 GetHeight() const = 0;

	/**
	 * Gets the total size of this buffer.
	 */
	virtual int64 GetSize() const = 0;

	/**
	 * Gets the Format of the buffer can be used to cast to the Specific type.
	 */
	virtual int32 GetFormat() const = 0;

	/**
	 * Gets a const pointer to the beginning of entire buffer for reading.
	 */
	const virtual uint8_t* GetData() const = 0;

	/**
	 * Gets a pointer to the beginning of the entire buffer for editing.
	 */
	virtual uint8_t* GetMutableData() = 0;
};
