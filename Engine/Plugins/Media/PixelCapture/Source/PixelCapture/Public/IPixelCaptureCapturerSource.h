// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"

/**
 * The interface that is queried when the Layered or Multi capturers wish to create specific
 * Capturers for a given format.
 */
class IPixelCaptureCapturerSource
{
public:
	/**
	 * Implement this to create an capture process that captures to the FinalFormat at
	 * the given resolution. The source format should be known by whatever is fed into the
	 * capture system.
	 * @param FinalFormat The format the capture process should capture.
	 * @param Resolution The frame resolution the capture process should output.
	 * @return The FPixelCaptureCapturer that will do the work.
	 */
	virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint OutputResolution) = 0;

	/**
	 * Implement this to create an capture process that captures to the FinalFormat at
	 * the given scale. The source format should be known by whatever is fed into the
	 * capture system.
	 * @param FinalFormat The format the capture process should capture.
	 * @param FinalScale The frame scale the capture process should scale to.
	 * @return The FPixelCaptureCapturer that will do the work.
	 */
	UE_DEPRECATED(5.7, "CreateCapturer using a scale has been deprecated. Please implement CreateCapturer using an FIntPoint output resolution.")
	virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, float FinalScale) final { return nullptr; };
};
