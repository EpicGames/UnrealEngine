// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureFrameMetadata.h"
#include "HAL/PlatformTime.h"

/**
 * The base interface that is fed into the capture system. This wrapper is fed into the
 * FPixelCaptureCapturer objects for each format/resolution. The capturer should then cast
 * to the known type fed into the system to unwrap the data.
 */
class IPixelCaptureInputFrame
{
public:
	IPixelCaptureInputFrame() 
	{ 
		Metadata.SourceTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64()); 
	}

	IPixelCaptureInputFrame(TSharedPtr<FPixelCaptureUserData> UserData) 
	{ 
		Metadata.SourceTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64()); 
		Metadata.UserData = UserData; 
	}

	virtual ~IPixelCaptureInputFrame() = default;

	/**
	 * Should return a unique type id from either EPixelCaptureBufferFormat or an
	 * extended user implemented enum. Value crashes could result in incorrect/unsafe casting.
	 * @return A unique id value for the implementation of this interface.
	 */
	virtual int32 GetType() const = 0;

	/**
	 * Gets the width of the input frame.
	 * @return The pixel width of the input frame.
	 */
	virtual int32 GetWidth() const = 0;

	/**
	 * Gets the height of the input frame.
	 * @return The pixel height of the input frame.
	 */
	virtual int32 GetHeight() const = 0;

	/**
	 * Internal structure that contains various bits of information about the capture process
	 */
	FPixelCaptureFrameMetadata Metadata;
};
