// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureFrameMetadata.h"

/**
 * Wraps the output of the capture process for a single layer. Extend this for your
 * own result types. You must implement GetWidth and GetHeight to return the width
 * and height of the frame. Add your own method to extract the captured data.
 */
class IPixelCaptureOutputFrame
{
public:
	virtual ~IPixelCaptureOutputFrame() = default;

	virtual int32 GetWidth() const = 0;
	virtual int32 GetHeight() const = 0;

	/**
	 * Internal structure that contains various bits of information about the capture and encode process
	 */
	FPixelCaptureFrameMetadata Metadata;
};
