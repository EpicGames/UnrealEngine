// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * This is a list of all internal buffer formats supported by Pixel Capture. To implement
 * your own buffer format value you must start your values at FORMAT_USER.
 */
namespace PixelCaptureBufferFormat
{
	static const int32 FORMAT_UNKNOWN = 0;
	static const int32 FORMAT_RHI = 1;
	static const int32 FORMAT_I420 = 2;
	static const int32 FORMAT_NV12 = 3;
	static const int32 FORMAT_P010 = 4;
	static const int32 FORMAT_I010 = 5;

	static const int32 FORMAT_USER = 128;
} // namespace PixelCaptureBufferFormat
