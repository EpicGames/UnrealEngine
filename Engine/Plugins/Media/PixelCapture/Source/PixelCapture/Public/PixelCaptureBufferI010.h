// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureBuffer.h"

#define UE_API PIXELCAPTURE_API

/**
 * Buffer class for holding I010 frame data.
 */
class FPixelCaptureBufferI010 : public IPixelCaptureBuffer
{
public:
	/**
	 * Create an empty buffer of the specified size.
	 */
	UE_API FPixelCaptureBufferI010(int InWidth, int InHeight);

	FPixelCaptureBufferI010(const FPixelCaptureBufferI010& Other) = default;
	virtual ~FPixelCaptureBufferI010() = default;

	/**
	 * Get the width of the frame
	 */
	virtual int32 GetWidth() const override { return Width; }

	/**
	 * Get the height of the frame
	 */
	virtual int32 GetHeight() const override { return Height; }

	/**
	 * Get the Buffer format
	 */
	virtual int32 GetFormat() const override { return PixelCaptureBufferFormat::FORMAT_I010; };

	/**
	 * Get the stride of the Y plane
	 */
	int32 GetStrideY() const { return StrideY; }

	/**
	 * Get the stride of the U and V planes
	 */
	int32 GetStrideUV() const { return StrideUV; }

	/**
	 * Gets a const pointer to the beginning of entire buffer for reading.
	 */
	UE_API virtual const uint8_t* GetData() const override;

	/**
	 * Gets a const pointer to the beginning of the Y plane for reading.
	 */
	UE_API const uint16_t* GetDataY() const;

	/**
	 * Gets a const pointer to the beginning of the U plane for reading.
	 */
	UE_API const uint16_t* GetDataU() const;

	/**
	 * Gets a const pointer to the beginning of the V plane for reading.
	 */
	UE_API const uint16_t* GetDataV() const;

	/**
	 * Gets a pointer to the beginning of the entire buffer for editing.
	 */
	UE_API virtual uint8_t* GetMutableData() override;

	/**
	 * Gets a pointer to the beginning of the Y plane for editing.
	 */
	UE_API uint16_t* GetMutableDataY();

	/**
	 * Gets a pointer to the beginning of the U plane for editing.
	 */
	UE_API uint16_t* GetMutableDataU();

	/**
	 * Gets a pointer to the beginning of the V plane for editing.
	 */
	UE_API uint16_t* GetMutableDataV();

	/**
	 * Gets the size of the Y plane in bytes.
	 */
	UE_API int32 GetDataSizeY() const;

	/**
	 * Gets the size of both the U and V planes in bytes.
	 */
	UE_API int32 GetDataSizeUV() const;

	/**
	 * Gets the total size of this buffer.
	 */
	virtual int64 GetSize() const override { return Data.Num(); }

	/**
	 * Copies the given buffer into this buffer. Supplied so copies
	 * can be explicit in code where we want to be verbose.
	 */
	void Copy(const FPixelCaptureBufferI010& Other) { *this = Other; }

private:
	int Width = 0;
	int Height = 0;

	int StrideY = 0;
	int StrideUV = 0;

	TArray<uint16_t> Data;
};

#undef UE_API
