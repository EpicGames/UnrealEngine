// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureBuffer.h"

#define UE_API PIXELCAPTURE_API

/**
 * Buffer class for holding I420 frame data.
 */
class FPixelCaptureBufferNV12  : public IPixelCaptureBuffer
{
public:
	/**
	 * Create an empty buffer of the specified size.
	 */
	UE_API FPixelCaptureBufferNV12(int InWidth, int InHeight);

	FPixelCaptureBufferNV12(const FPixelCaptureBufferNV12& Other) = default;
	virtual	~FPixelCaptureBufferNV12() = default;

	/**
	 * Get the width of the frame
	 */
	virtual int GetWidth() const override { return Width; }

	/**
	 * Get the height of the frame
	 */
	virtual int GetHeight() const override { return Height; }

	/**
	 * Get the Buffer format
	 */
	virtual int32 GetFormat() const override { return PixelCaptureBufferFormat::FORMAT_NV12; };

	/**
	 * Get the stride of the Y plane
	 */
	int GetStrideY() const { return StrideY; }

	/**
	 * Get the stride of the U and V planes
	 */
	int GetStrideUV() const { return StrideUV; }

	/**
	 * Gets a const pointer to the beginning of entire buffer for reading.
	 */
	UE_API virtual const uint8_t* GetData() const override;

	/**
	 * Gets a const pointer to the beginning of the Y plane for reading.
	 */
	UE_API const uint8_t* GetDataY() const;

	/**
	 * Gets a const pointer to the beginning of the U plane for reading.
	 */
	UE_API const uint8_t* GetDataUV() const;

	/**
	 * Gets a pointer to the beginning of the entire buffer for editing.
	 */
	UE_API virtual uint8_t* GetMutableData() override;

	/**
	 * Gets a pointer to the beginning of the Y plane for editing.
	 */
	UE_API uint8_t* GetMutableDataY();

	/**
	 * Gets a pointer to the beginning of the U plane for editing.
	 */
	UE_API uint8_t* GetMutableDataUV();

	/**
	 * Gets the size of the Y plane in bytes.
	 */
	UE_API int GetDataSizeY() const;

	/**
	 * Gets the size of both the UV plane in bytes.
	 */
	UE_API int GetDataSizeUV() const;

	/**
	 * Gets the total size of this buffer.
	 */
	virtual int64 GetSize() const override { return Data.Num(); }

	/**
	 * Copies the given buffer into this buffer. Supplied so copies
	 * can be explicit in code where we want to be verbose.
	 */
	void Copy(const FPixelCaptureBufferNV12& Other) { *this = Other; }

private:
	int Width = 0;
	int Height = 0;

	int StrideY = 0;
	int StrideUV = 0;

	TArray<uint8_t> Data;
};

#undef UE_API
