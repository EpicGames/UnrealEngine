// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureInputFrame.h"
#include "IPixelCaptureOutputFrame.h"
#include "HAL/ThreadSafeCounter64.h"

#define UE_API PIXELCAPTURE_API

namespace UE::PixelCapture
{
	class FOutputFrameBuffer;

	PIXELCAPTURE_API void MarkCPUWorkStart(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
	PIXELCAPTURE_API void MarkCPUWorkEnd(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
	PIXELCAPTURE_API void MarkGPUWorkStart(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
	PIXELCAPTURE_API void MarkGPUWorkEnd(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
} // namespace UE::PixelCapture

struct FPixelCaptureCapturerConfig
{
	FIntPoint OutputResolution;
	bool	  bIsSRGB = false;
};
/**
 * The base class for all Capturers in the system.
 * Incoming frames will be user types implementing IPixelCaptureInputFrame.
 * Outgoing frames should be user types implementing IPixelCaptureOutputFrame.
 * Each capturer system should expect one known input user type.
 * Implement CreateOutputBuffer to create your custom IPixelCaptureOutputFrame
 * implementation to hold the result of the capture process.
 * Implement BeginProcess to start the capture work which ideally should be
 * an async task of some sort.
 * The capture work should fill the given IPixelCaptureOutputFrame and then
 * call EndProcess to indicate the work is done.
 * While the capture should be async it should only expect to work on one
 * frame at a time.
 */
class FPixelCaptureCapturer
{
public:
	UE_API FPixelCaptureCapturer(FPixelCaptureCapturerConfig Config = {});
	UE_API virtual ~FPixelCaptureCapturer();

	/**
	 * Called when an input frame needs capturing.
	 * @param InputFrame The input frame to be captured.
	 */
	UE_API void Capture(const IPixelCaptureInputFrame& InputFrame);

	/**
	 * Returns true if Initialize() has been called.
	 * Output data can depend on the incoming frames so we do lazy initialization when we first consume.
	 * @return True if this process has been initialized correctly.
	 */
	bool IsInitialized() const { return bInitialized; }

	/**
	 * Returns true when this process is actively working on capturing frame data.
	 * @return True when this process is busy.
	 */
	bool IsBusy() const { return bBusy; }

	/**
	 * Returns true if this process has a frame in the output buffer ready to be read.
	 * @return True when this process has output data.
	 */
	bool HasOutput() const { return bHasOutput; }

	/**
	 * Gets the output frame from the output buffer.
	 * @return The output data of this process.
	 */
	UE_API TSharedPtr<IPixelCaptureOutputFrame> ReadOutput();

	/**
	 * Listen on this to be notified when the capture process completes for each input.
	 */
	DECLARE_TS_MULTICAST_DELEGATE(FOnComplete);
	FOnComplete OnComplete;

protected:
	/**
	 * Sets whether this process is actively working on capturing frame data.
	 * @param bInBusy whether this process is busy.
	 */
	void SetIsBusy(bool bInBusy) { bBusy = bInBusy; }

	/**
	 * Gets the human readable name for this capture process. This name will be used in stats
	 * readouts so the shorter the better.
	 * @return A human readable name for this capture process.
	 */
	virtual FString GetCapturerName() const = 0;

	/**
	 * Initializes the process to be ready for work. Called once at startup.
	 * @param InputWidth The pixel count of the input frame width
	 * @param InputHeight The pixel count of the input frame height
	 */
	UE_API virtual void Initialize(int32 InputWidth, int32 InputHeight);

	/**
	 * Implement this to create a buffer for the output.
	 * @param InputWidth The pixel width of the input frame.
	 * @param InputHeight The pixel height of the input frame.
	 * @return An empty output structure that the process can store the output of its process on.
	 */
	virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) = 0;

	/**
	 * Implement this with your specific process to capture the incoming frame.
	 * @param InputFrame The input frame data for the process to begin working on.
	 * @param OutputBuffer The destination buffer for the process. Is guaranteed to be of the type created in CreateOutputBuffer()
	 */
	UE_DEPRECATED(5.6, "BeginProcess(const IPixelCaptureInputFrame& InputFrame, IPixelCaptureOutputFrame* OutputBuffer) has been deprecated. Please use BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) to ensure memory safety.")
	void		 BeginProcess(const IPixelCaptureInputFrame& InputFrame, IPixelCaptureOutputFrame* OutputBuffer) {};
	virtual void BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) = 0;

	/**
	 * Metadata info (optional). Marks the start of some CPU work. Multiple work sections are valid.
	 */
	UE_DEPRECATED(5.6, "MarkCPUWorkStart(void) has been deprecated. Please use MarkCPUWorkStart(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) to ensure the metadata is updated for the correct output buffer.")
	void MarkCPUWorkStart() {}

	/**
	 * Metadata info (optional). Marks the end of some CPU work.
	 */
	UE_DEPRECATED(5.6, "MarkCPUWorkEnd(void) has been deprecated. Please use UE::PixelCapture::MarkCPUWorkEnd(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) to ensure the metadata is updated for the correct output buffer.")
	void MarkCPUWorkEnd() {}

	/**
	 * Metadata info (optional). Marks the start of some GPU work. Multiple work sections are valid.
	 */
	UE_DEPRECATED(5.6, "MarkGPUWorkStart(void) has been deprecated. Please use UE::PixelCapture::MarkGPUWorkStart(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) to ensure the metadata is updated for the correct output buffer.")
	void MarkGPUWorkStart() {}

	/**
	 * Metadata info (optional). Marks the end of some GPU work.
	 */
	UE_DEPRECATED(5.6, "MarkGPUWorkEnd(void) has been deprecated. Please use UE::PixelCapture::MarkGPUWorkEnd(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) to ensure the metadata is updated for the correct output buffer.")
	void MarkGPUWorkEnd() {}

	/**
	 * Call this to mark the end of processing. Will commit the current write buffer into the read buffer.
	 */
	UE_DEPRECATED(5.6, "EndProcess(void) has been deprecated. Please use EndProcess(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) to ensure the metadata is updated for the correct output buffer.")
	void EndProcess() {}
	UE_API void EndProcess(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);

protected:
	FPixelCaptureCapturerConfig Config;

private:
	bool bInitialized = false;
	bool bBusy = false;
	bool bHasOutput = false;

	int32 ExpectedInputWidth = 0;
	int32 ExpectedInputHeight = 0;

	FThreadSafeCounter64 FrameId;

	TUniquePtr<UE::PixelCapture::FOutputFrameBuffer> Buffer;

	mutable FCriticalSection					 OutputBuffersMutex;
	TArray<TSharedPtr<IPixelCaptureOutputFrame>> OutputBuffers;

	UE_API void InitMetadata(FPixelCaptureFrameMetadata InputMetadata, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
	UE_API void FinalizeMetadata(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
};

#undef UE_API
