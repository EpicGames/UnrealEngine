// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer that will copy I420 frames.
 * Input: FPixelCaptureInputFrameI420
 * Output: FPixelCaptureOutputFrameI420
 */
class FPixelCaptureCapturerI420 : public FPixelCaptureCapturer
{
public:
	static UE_API TSharedPtr<FPixelCaptureCapturerI420> Create(FPixelCaptureCapturerConfig Config = {});
	virtual ~FPixelCaptureCapturerI420() = default;

protected:
	virtual FString					  GetCapturerName() const override { return "I420 Copy"; }
	UE_API virtual void					  Initialize(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void					  BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	UE_API FPixelCaptureCapturerI420(FPixelCaptureCapturerConfig& Config);
};

#undef UE_API
