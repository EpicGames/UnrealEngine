// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer that will capture I420 frames to native RHI textures.
 * Input: FPixelCaptureInputFrameI420
 * Output: FPixelCaptureOutputFrameRHI
 */
class FPixelCaptureCapturerI420ToRHI : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerI420ToRHI>
{
public:
	static UE_API TSharedPtr<FPixelCaptureCapturerI420ToRHI> Create(FPixelCaptureCapturerConfig Config = {});
	UE_API virtual ~FPixelCaptureCapturerI420ToRHI();

protected:
	virtual FString GetCapturerName() const override { return "I420ToRHI"; }
	UE_API virtual void Initialize(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	uint8_t* ARGBBuffer = nullptr;

	UE_API FPixelCaptureCapturerI420ToRHI(FPixelCaptureCapturerConfig& Config);
};

#undef UE_API
