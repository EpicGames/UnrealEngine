// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureCapturerSource.h"

/**
 * A capturer that contains multiple layers of different resolution capture processes.
 * Feed it a IPixelCaptureCapturerSource that will create the appropriate base capturers.
 * Input: User defined
 * Output: Capturer defined
 */
class FPixelCaptureCapturerLayered : public TSharedFromThis<FPixelCaptureCapturerLayered>
{
public:
	/**
	 * Create a new Layered Capturer.
	 * @param InCapturerSource A source for capturers for each layer.
	 * @param InDestinationFormat The format to capture to.
	 * @param InOutputResolutions An optional parameter to specify the resolutions to create capture processes for during construction.
	 */
	static TSharedPtr<FPixelCaptureCapturerLayered> Create(IPixelCaptureCapturerSource* InCapturerSource, int32 InDestinationFormat, TArray<FIntPoint> InOutputResolutions = {});

	/**
	 * Create a new Layered Capturer.
	 * @param InCapturerSource A source for capturers for each layer.
	 * @param InDestinationFormat The format to capture to.
	 * @param LayerScales A list of scales for each layer.
	 */
	UE_DEPRECATED(5.7, "Create taking in an array of layer scales has been deprecated. You no longer need to pass in the scales during construction!")
	static TSharedPtr<FPixelCaptureCapturerLayered> Create(IPixelCaptureCapturerSource* InCapturerSource, int32 InDestinationFormat, TArray<float> LayerScales);

	virtual ~FPixelCaptureCapturerLayered() = default;

	/**
	 * Begins the capture process of a given frame.
	 */
	void Capture(const IPixelCaptureInputFrame& SourceFrame);

	/**
	 * Try to read the result of the capture process. May return null if no output
	 * has been captured yet.
	 * @param LayerSize The resolution of the layer to try and read the output from.
	 * @return The captured frame layer if one exists. Null otherwise.
	 */
	TSharedPtr<IPixelCaptureOutputFrame> ReadOutput(FIntPoint LayerSize);

	/**
	 * Try to read the result of the capture process. May return null if no output
	 * has been captured yet.
	 * @param LayerIndex The layer to try and read the output from.
	 * @return The captured frame layer if one exists. Null otherwise.
	 */
	UE_DEPRECATED(5.7, "ReadOutput using a layer index has been deprecated. Please use ReadOutput that takes a resolution")
	TSharedPtr<IPixelCaptureOutputFrame> ReadOutput(int32 LayerIndex) { return nullptr; }

	/**
	 * A callback to broadcast on when the frame has completed the capture process.
	 * Called once when all layers of a given input frame have completed.
	 */
	DECLARE_TS_MULTICAST_DELEGATE(FOnComplete);
	FOnComplete OnComplete;

protected:
	FPixelCaptureCapturerLayered(IPixelCaptureCapturerSource* InCapturerSource, int32 InDestinationFormat);

	void OnCaptureComplete();

	IPixelCaptureCapturerSource* CapturerSource;
	int32 DestinationFormat;
	TMap<FIntPoint, TSharedPtr<FPixelCaptureCapturer>> LayerCapturers;
	mutable FCriticalSection LayersGuard;
};
