// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureCapturerSource.h"
#include "PixelCaptureCapturerLayered.h"

#define UE_API PIXELCAPTURE_API

/**
 * A capturer that contains multiple formats of multi layer capture processes.
 * Feed it a IPixelCaptureCapturerSource that will create the appropriate base capturers.
 * Input: User defined
 * Output: Capturer defined
 */
class FPixelCaptureCapturerMultiFormat : public TSharedFromThis<FPixelCaptureCapturerMultiFormat>
{
public:
	/**
	 * Create a new multi-format multi-Layered Capturer.
	 * @param InCapturerSource A source for capturers for each layer.
	 * @param InOutputResolutions An optional parameter containing the output resolutions if known.
	 */
	static UE_API TSharedPtr<FPixelCaptureCapturerMultiFormat> Create(IPixelCaptureCapturerSource* InCapturerSource, TArray<FIntPoint> InOutputResolutions = {});

	/**
	 * Create a new multi-format multi-Layered Capturer.
	 * @param InCapturerSource A source for capturers for each layer.
	 * @param LayerScales A list of scales for each layer.
	 */
	UE_DEPRECATED(5.7, "Create taking in an array of layer scales has been deprecated. You no longer need to pass in the scales during construction!")
	static UE_API TSharedPtr<FPixelCaptureCapturerMultiFormat> Create(IPixelCaptureCapturerSource* InCapturerSource, TArray<float> LayerScales);

	UE_API virtual ~FPixelCaptureCapturerMultiFormat();

	/**
	 * Gets the number of layers in the multi-layered capturers.
	 * @return The number of layers in each multi-layered capturer.
	 */
	int32 GetNumLayers() const { return LayerSizes.Num(); }

	/**
	 * Gets the layer sizes
	 * @return The array containing the resolution of each layer
	 */
	TArray<FIntPoint> GetLayerSizes() const { return LayerSizes.Array(); }

	/**
	 * Gets the frame width of a given output layer.
	 * @return The pixel count of the width of a output layer.
	 */
	UE_DEPRECATED(5.7, "GetWidth for a layer index has been deprecated. You shouldn't need to use this as PixelCapture now uses resolutions directly")
	int32 GetWidth(int LayerIndex) const { return 0; }

	/**
	 * Gets the frame height of a given output layer.
	 * @return The pixel count of the height of a output layer.
	 */
	UE_DEPRECATED(5.7, "GetHeight for a layer index has been deprecated. You shouldn't need to use this as PixelCapture now uses resolutions directly")
	int32 GetHeight(int LayerIndex) const { return 0; }

	/**
	 * Begins the capture process of a given frame.
	 */
	UE_API void Capture(const IPixelCaptureInputFrame& SourceFrame);

	/**
	 * Sets up a capture pipeline for the given destination format. No effect if the
	 * pipeline already exists.
	 * @param Format The destination format for the requested pipeline.
	 */
	UE_API void AddOutputFormat(int32 Format);

	/**
	 * Requests the output in a specific format. If this is the first request for
	 * the format and AddOutputFormat has not been called for the format then this
	 * call will return nullptr. Otherwise will return the buffer for the format
	 * provided the capture has completed.
	 * @param Format The format we want the output in.
	 * @param LayerSize The resolution of the layer we want to get the output of.
	 * @return The final buffer of a given format if it exists or null.
	 */
	UE_API TSharedPtr<IPixelCaptureOutputFrame> RequestFormat(int32 Format, FIntPoint LayerSize);

	/**
	 * Requests the output in a specific format. If this is the first request for
	 * the format and AddOutputFormat has not been called for the format then this
	 * call will return nullptr. Otherwise will return the buffer for the format
	 * provided the capture has completed.
	 * @param Format The format we want the output in.
	 * @param LayerIndex The layer we want to get the output of.
	 * @return The final buffer of a given format if it exists or null.
	 */
	UE_DEPRECATED(5.7, "RequestFormat taking a layer index has been deprecated. Please use RequestFormat that takes an FIntPoint resolution")
	TSharedPtr<IPixelCaptureOutputFrame> RequestFormat(int32 Format, int32 LayerIndex) { return nullptr; }

	/**
	 * Like RequestFormat except if the format does not exist it will add it and then
	 * wait for the format to have output.
	 * NOTE: This will block the calling thread so it is important that the capture
	 * process is not dependent on this calling thread or we will deadlock.
	 * @param Format The format we want the output in.
	 * @param LayerSize The resolution of the layer we want to get the output of.
	 * @param MaxWaitTime The max number of milliseconds to wait for a frame. Default is 5 seconds.
	 * @return The final buffer of a given format or null in case of timeout or the capturer has been disconnected.
	 */
	UE_API TSharedPtr<IPixelCaptureOutputFrame> WaitForFormat(int32 Format, FIntPoint LayerSize, uint32 MaxWaitTime = 5000);

	/**
	 * Like RequestFormat except if the format does not exist it will add it and then
	 * wait for the format to have output.
	 * NOTE: This will block the calling thread so it is important that the capture
	 * process is not dependent on this calling thread or we will deadlock.
	 * @param Format The format we want the output in.
	 * @param LayerIndex The layer we want to get the output of.
	 * @param MaxWaitTime The max number of milliseconds to wait for a frame. Default is 5 seconds.
	 * @return The final buffer of a given format or null in case of timeout or the capturer has been disconnected.
	 */
	UE_DEPRECATED(5.7, "WaitForFormat taking a layer index has been deprecated. Please use WaitForFormat that takes an FIntPoint resolution")
	TSharedPtr<IPixelCaptureOutputFrame> WaitForFormat(int32 Format, int32 LayerIndex, uint32 MaxWaitTime = 5000) { return nullptr; }

	/**
	 * Call to notify this capturer that it has been disconnected and no more frames
	 * will be captured and any waiting for format calls should stop waiting.
	 */
	UE_API void OnDisconnected();

	/**
	 * Listen on this to be notified when a frame completes all capture formats/layers.
	 */
	DECLARE_TS_MULTICAST_DELEGATE(FOnComplete);
	FOnComplete OnComplete;

protected:
	UE_API FPixelCaptureCapturerMultiFormat(IPixelCaptureCapturerSource* InCapturerSource, TArray<FIntPoint> InOutputResolutions);
	UE_API void OnCaptureFormatComplete(int32 Format);

	UE_API FEvent* GetEventForFormat(int32 Format);
	UE_API void	CheckFormatEvent(int32 Format);
	UE_API void	FreeEvent(int32 Format, FEvent* Event);
	UE_API void	FlushWaitingEvents();

	IPixelCaptureCapturerSource* CapturerSource;

	mutable FCriticalSection LayersGuard;
	TSet<FIntPoint> LayerSizes;

	TMap<int32, TSharedPtr<FPixelCaptureCapturerLayered>> FormatCapturers;
	TAtomic<int>										  PendingFormats = 0; // atomic because the complete events can come in on multiple threads
	mutable FCriticalSection							  FormatGuard;

	FCriticalSection	 EventMutex;
	TMap<int32, FEvent*> FormatEvents;

	bool bDisconnected = false;
};

#undef UE_API
