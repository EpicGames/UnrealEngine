// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer that will capture NV12 frames to native RHI textures.
 * Input: FPixelCaptureInputFrameNV12
 * Output: FPixelCaptureOutputFrameRHI
 */
class FPixelCaptureCapturerNV12ToRHI : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerNV12ToRHI>
{
public:
	static UE_API TSharedPtr<FPixelCaptureCapturerNV12ToRHI> Create(FPixelCaptureCapturerConfig Config = {});
	UE_API virtual ~FPixelCaptureCapturerNV12ToRHI();

protected:
	virtual FString GetCapturerName() const override { return "NV12ToRHI"; }
	UE_API virtual void Initialize(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	uint8_t* R8Buffer = nullptr;

	UE_API FPixelCaptureCapturerNV12ToRHI(FPixelCaptureCapturerConfig& Config);
};

#undef UE_API
