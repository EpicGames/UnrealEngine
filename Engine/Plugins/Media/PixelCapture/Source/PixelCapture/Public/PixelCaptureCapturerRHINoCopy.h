// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"
#include "RHI.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer for receiving copy-safe RHI Texture frames. This pixel capturer will shortcut the need to copy texture is the input
 * and output frame sizes match
 *
 * Input: FPixelCaptureInputFrameRHI
 * Output: FPixelCaptureOutputFrameRHI
 */
class FPixelCaptureCapturerRHINoCopy : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerRHINoCopy>
{
public:
	/**
	 * Creates a new capturer that assigns the output frame to the input frame. NOTE: This means this capturer doesn't support resolution or format changes
	 */
	static UE_API TSharedPtr<FPixelCaptureCapturerRHINoCopy> Create(FPixelCaptureCapturerConfig Config = {});

	UE_DEPRECATED(5.7, "Create taking a scale has been deprecated. Please use Create that takes an FPixelCaptureCapturerConfig instead")
	static TSharedPtr<FPixelCaptureCapturerRHINoCopy> Create(float InScale) { return nullptr; }

	virtual ~FPixelCaptureCapturerRHINoCopy() = default;

protected:
	virtual FString					  GetCapturerName() const override { return "RHI No Copy"; }
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void					  BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	UE_API FPixelCaptureCapturerRHINoCopy(FPixelCaptureCapturerConfig& Config);
};

#undef UE_API
