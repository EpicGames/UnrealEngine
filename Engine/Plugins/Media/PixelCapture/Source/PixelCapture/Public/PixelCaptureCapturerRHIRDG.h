// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"
#include "RHI.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer that will copy RHI texture frames.
 * Input: FPixelCaptureInputFrameRHI
 * Output: FPixelCaptureOutputFrameRHI
 */
class FPixelCaptureCapturerRHIRDG : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerRHIRDG>
{
public:
	static UE_API TSharedPtr<FPixelCaptureCapturerRHIRDG> Create(FPixelCaptureCapturerConfig Config = {});

	UE_DEPRECATED(5.7, "Create taking a scale has been deprecated. Please use Create that takes an FPixelCaptureCapturerConfig instead")
	static TSharedPtr<FPixelCaptureCapturerRHIRDG> Create(float InScale) { return nullptr; }

	virtual ~FPixelCaptureCapturerRHIRDG() = default;

protected:
	virtual FString					  GetCapturerName() const override { return "RHI RDG Copy"; }
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void					  BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	UE_API FPixelCaptureCapturerRHIRDG(FPixelCaptureCapturerConfig& Config);
	UE_API void CheckComplete();
	UE_API void OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
};

#undef UE_API
