// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"
#include "RHI.h"
#include "RHIGPUReadback.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer that will capture RHI texture frames to I420 buffers
 * utilizing cpu functions. Involves CPU readback of GPU textures and processing
 * of that readback data. Input: FPixelCaptureInputFrameRHI Output:
 * FPixelCaptureOutputFrameI420
 */
class FPixelCaptureCapturerRHIToI420CPU : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerRHIToI420CPU>
{
public:
	static UE_API TSharedPtr<FPixelCaptureCapturerRHIToI420CPU> Create(FPixelCaptureCapturerConfig Config = {});

	UE_DEPRECATED(5.7, "Create taking a scale has been deprecated. Please use Create that takes an FPixelCaptureCapturerConfig instead")
	static TSharedPtr<FPixelCaptureCapturerRHIToI420CPU> Create(float InScale) { return nullptr; }

	UE_API virtual ~FPixelCaptureCapturerRHIToI420CPU();

protected:
	virtual FString					  GetCapturerName() const override { return "RHIToI420CPU"; }
	UE_API virtual void					  Initialize(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void					  BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	FTextureRHIRef StagingTexture;

	UE_API FPixelCaptureCapturerRHIToI420CPU(FPixelCaptureCapturerConfig& Config);

#if 1
	TSharedPtr<FRHIGPUTextureReadback> TextureReader;
	UE_API void							   OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
	UE_API void							   CheckComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
#else
	FTextureRHIRef ReadbackTexture;
#endif
};

#undef UE_API
