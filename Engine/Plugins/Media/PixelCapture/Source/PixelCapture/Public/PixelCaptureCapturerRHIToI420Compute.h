// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturer.h"
#include "RHI.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic capturer that will capture RHI texture frames to I420 buffers utilizing a compute shader.
 * Involves compute shader reading and processing of GPU textures so should be faster than CPU variant.
 * Input: FPixelCaptureInputFrameRHI
 * Output: FPixelCaptureOutputFrameI420
 */
class FPixelCaptureCapturerRHIToI420Compute : public FPixelCaptureCapturer, public TSharedFromThis<FPixelCaptureCapturerRHIToI420Compute>
{
public:
	static UE_API TSharedPtr<FPixelCaptureCapturerRHIToI420Compute> Create(FPixelCaptureCapturerConfig Config = {});

	UE_DEPRECATED(5.7, "Create taking a scale has been deprecated. Please use Create that takes an FPixelCaptureCapturerConfig instead")
	static TSharedPtr<FPixelCaptureCapturerRHIToI420Compute> Create(float InScale) { return nullptr; }

	UE_API virtual ~FPixelCaptureCapturerRHIToI420Compute();

protected:
	virtual FString					  GetCapturerName() const override { return "RHIToI420Compute"; }
	UE_API virtual void					  Initialize(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual IPixelCaptureOutputFrame* CreateOutputBuffer(int32 InputWidth, int32 InputHeight) override;
	UE_API virtual void					  BeginProcess(const IPixelCaptureInputFrame& InputFrame, TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer) override;

private:
	// dimensions of the texures
	FIntPoint PlaneYDimensions;
	FIntPoint PlaneUVDimensions;

	// used as targets for the compute shader
	FTextureRHIRef TextureY;
	FTextureRHIRef TextureU;
	FTextureRHIRef TextureV;

	// the UAVs of the targets
	FUnorderedAccessViewRHIRef TextureYUAV;
	FUnorderedAccessViewRHIRef TextureUUAV;
	FUnorderedAccessViewRHIRef TextureVUAV;

	// cpu readable copies of the targets above
	FTextureRHIRef StagingTextureY;
	FTextureRHIRef StagingTextureU;
	FTextureRHIRef StagingTextureV;

	// memory mapped pointers of the staging textures
	void* MappedY = nullptr;
	void* MappedU = nullptr;
	void* MappedV = nullptr;

	int32 YStride = 0;
	int32 UStride = 0;
	int32 VStride = 0;

	UE_API FPixelCaptureCapturerRHIToI420Compute(FPixelCaptureCapturerConfig& Config);
	UE_API void OnRHIStageComplete(TSharedPtr<IPixelCaptureOutputFrame> OutputBuffer);
	UE_API void CleanUp();
};

#undef UE_API
