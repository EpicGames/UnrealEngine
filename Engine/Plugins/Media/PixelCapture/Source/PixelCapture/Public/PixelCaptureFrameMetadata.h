// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPixelCaptureUserData{};

struct FPixelCaptureFrameMetadata
{
	// Identifier for the capture pipeline/process this frame took
	FString ProcessName = "Unknown";

	// Identifier for the frame
	uint64 Id = 0;

	// Which layer this specific frame is associated with
	int32 Layer = 0;

	// The time this frame was sourced/created
	uint64 SourceTime = 0;

	// Timestamps to track the entire length of the capture process
	uint64 CaptureStartCyles = 0;
	uint64 CaptureEndCyles = 0;

	// Timestamps for when CPU work has started / ended
	uint64 CaptureProcessCPUStartCycles = 0;
	uint64 CaptureProcessCPUEndCycles = 0;

	// Timestamps for when the capture process was enqueued to the GPU and when the GPU actually started work
	uint64 CaptureProcessGPUEnqueueStartCycles = 0;
	uint64 CaptureProcessGPUEnqueueEndCycles = 0;

	// Timestamps for when the GPU started work and when the GPU finished the capture process
	uint64 CaptureProcessGPUStartCycles = 0;
	uint64 CaptureProcessGPUEndCycles = 0;

	// Timestamps for tracking time after the GPU completed it's work until the capture process completed.
	uint64 CaptureProcessPostGPUStartCycles = 0;
	uint64 CaptureProcessPostGPUEndCycles = 0;

	// Capture process timings. Duration not timestamp.
	UE_DEPRECATED(5.6, "CaptureTime has been deprecated. You can calculate the time taken with FPlatformTime::ToMilliseconds64(CaptureEndCyles - CaptureStartCyles).")
	uint64 CaptureTime = 0;
	UE_DEPRECATED(5.6, "CaptureProcessCPUTime has been deprecated. You can calculate the time taken with FPlatformTime::ToMilliseconds64(CaptureProcessCPUEndCycles - CaptureProcessCPUStartCycles).")
	uint64 CaptureProcessCPUTime = 0;
	UE_DEPRECATED(5.6, "CaptureProcessGPUDelay has been deprecated. You can calculate the time taken with FPlatformTime::ToMilliseconds64(CaptureProcessGPUEnqueueEndCycles - CaptureProcessGPUEnqueueStartCycles).")
	uint64 CaptureProcessGPUDelay = 0;
	UE_DEPRECATED(5.6, "CaptureProcessGPUTime has been deprecated. You can calculate the time taken with FPlatformTime::ToMilliseconds64(CaptureProcessGPUEndCycles - CaptureProcessGPUStartCycles).")
	uint64 CaptureProcessGPUTime = 0;

	// Display process timings. Duration not timestamp.
	uint64 DisplayTime = 0;

	// Frame use timings (can happen multiple times. ie. we are consuming frames faster than producing them)
	uint32 UseCount = 0; // how many times the frame has been fed to the encoder or decoder

	// Encode Timings
	uint64 FirstEncodeStartTime = 0;
	uint64 LastEncodeStartTime = 0;
	uint64 LastEncodeEndTime = 0;

	// Packet Timings
	uint64 FirstPacketizationStartTime = 0;
	uint64 LastPacketizationStartTime = 0;
	uint64 LastPacketizationEndTime = 0;

	// Decode Timings
	uint64 FirstDecodeStartTime = 0;
	uint64 LastDecodeStartTime = 0;
	uint64 LastDecodeEndTime = 0;

	// Pointer to a user class that stores additional metadata that they want to keep throughout the process
	TSharedPtr<FPixelCaptureUserData> UserData;

	// Disable deprecations about accessing the deprecated members during copy
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// wanted this to be explicit with a name
	FPixelCaptureFrameMetadata Copy() const
	{
		return *this;
	}

	FPixelCaptureFrameMetadata() = default;
	FPixelCaptureFrameMetadata(FPixelCaptureFrameMetadata&&) = default;
	FPixelCaptureFrameMetadata(const FPixelCaptureFrameMetadata&) = default;
	FPixelCaptureFrameMetadata& operator=(FPixelCaptureFrameMetadata&&) = default;
	FPixelCaptureFrameMetadata& operator=(const FPixelCaptureFrameMetadata&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
