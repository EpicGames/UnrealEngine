// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureBufferI420.h"

UE_DEPRECATED(5.2, "Use FPixelCaptureBufferI420 instead.")
typedef FPixelCaptureBufferI420 FPixelCaptureI420Buffer;