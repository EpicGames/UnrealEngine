// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureInputFrame.h"
#include "PixelCaptureBufferI420.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic input frame for the Capture system that wraps a I420 buffer.
 */
class FPixelCaptureInputFrameI420 : public IPixelCaptureInputFrame
{
public:
	UE_API FPixelCaptureInputFrameI420(TSharedPtr<FPixelCaptureBufferI420> Buffer, TSharedPtr<FPixelCaptureUserData> UserData = nullptr);
	virtual ~FPixelCaptureInputFrameI420() = default;

	UE_API virtual int32 GetType() const override;
	UE_API virtual int32 GetWidth() const override;
	UE_API virtual int32 GetHeight() const override;

	UE_API TSharedPtr<FPixelCaptureBufferI420> GetBuffer() const;

private:
	TSharedPtr<FPixelCaptureBufferI420> I420Buffer;
};

#undef UE_API
