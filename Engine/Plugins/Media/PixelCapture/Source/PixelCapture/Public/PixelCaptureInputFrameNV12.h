// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureInputFrame.h"
#include "PixelCaptureBufferNV12.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic input frame for the Capture system that wraps a I420 buffer.
 */
class FPixelCaptureInputFrameNV12 : public IPixelCaptureInputFrame
{
public:
	UE_API FPixelCaptureInputFrameNV12(TSharedPtr<FPixelCaptureBufferNV12> Buffer, TSharedPtr<FPixelCaptureUserData> UserData = nullptr);
	virtual ~FPixelCaptureInputFrameNV12() = default;

	UE_API virtual int32 GetType() const override;
	UE_API virtual int32 GetWidth() const override;
	UE_API virtual int32 GetHeight() const override;

	UE_API TSharedPtr<FPixelCaptureBufferNV12> GetBuffer() const;

private:
	TSharedPtr<FPixelCaptureBufferNV12> NV12Buffer;
};

#undef UE_API
