// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureInputFrame.h"
#include "RHI.h"

#define UE_API PIXELCAPTURE_API

/**
 * A basic input frame for the Capture system that wraps a RHI texture buffer.
 */
class FPixelCaptureInputFrameRHI : public IPixelCaptureInputFrame
{
public:
	UE_API FPixelCaptureInputFrameRHI(FTextureRHIRef InFrameTexture, TSharedPtr<FPixelCaptureUserData> UserData = nullptr);
	virtual ~FPixelCaptureInputFrameRHI() = default;

	UE_API virtual int32 GetType() const override;
	UE_API virtual int32 GetWidth() const override;
	UE_API virtual int32 GetHeight() const override;

	FTextureRHIRef FrameTexture;
};

#undef UE_API
