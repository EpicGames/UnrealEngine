// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureOutputFrame.h"
#include "PixelCaptureBufferI420.h"

/**
 * A basic output frame from the Capture system that wraps a I420 buffer.
 */
class FPixelCaptureOutputFrameI420 : public IPixelCaptureOutputFrame
{
public:
	FPixelCaptureOutputFrameI420(TSharedPtr<FPixelCaptureBufferI420> InI420Buffer)
		: I420Buffer(InI420Buffer)
	{
	}
	virtual ~FPixelCaptureOutputFrameI420() = default;

	virtual int32 GetWidth() const override { return I420Buffer->GetWidth(); }
	virtual int32 GetHeight() const override { return I420Buffer->GetHeight(); }

	TSharedPtr<FPixelCaptureBufferI420> GetI420Buffer() const { return I420Buffer; }
	void SetI420Buffer(TSharedPtr<FPixelCaptureBufferI420> InI420Buffer) { I420Buffer = InI420Buffer; }

private:
	TSharedPtr<FPixelCaptureBufferI420> I420Buffer;
};
