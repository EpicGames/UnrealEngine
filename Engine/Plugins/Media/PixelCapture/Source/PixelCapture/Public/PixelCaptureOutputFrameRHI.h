// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureOutputFrame.h"
#include "RHI.h"

/**
 * A basic output frame from the Capture system that wraps a RHI texture buffer.
 */
class FPixelCaptureOutputFrameRHI : public IPixelCaptureOutputFrame
{
public:
	/**
	 * Sometimes we need to pack data in a pixel format not fully supported by the
	 * RHI, so we may instead store it as a different format for transport and read it as the 
	 * original format sometime later, this struct is meant to hold information
	 * about the wrapper format if it exists
	 */
	struct WrapperFormatData
	{
		int32 Width;
		int32 Height;
		EPixelFormat Format;
	};

	FPixelCaptureOutputFrameRHI(FTextureRHIRef InFrameTexture)
		: FrameTexture(InFrameTexture), bHasWrapperFormatData(false), WrapperData()
	{
	}

	FPixelCaptureOutputFrameRHI(FTextureRHIRef InFrameTexture, WrapperFormatData InWrapperData)
		: FrameTexture(InFrameTexture), bHasWrapperFormatData(true), WrapperData(InWrapperData)
	{
	}

	virtual ~FPixelCaptureOutputFrameRHI() = default;

	virtual int32 GetWidth() const override 
	{
		return bHasWrapperFormatData ? WrapperData.Width : FrameTexture->GetDesc().Extent.X;
	}
	virtual int32 GetHeight() const override
	{
		return bHasWrapperFormatData ? WrapperData.Height : FrameTexture->GetDesc().Extent.Y;
	}

	virtual bool GetWrapperFormatData(WrapperFormatData& OutData) const { 
		if(bHasWrapperFormatData)
		{
			OutData = WrapperData;
			return true;
		}

		return false;
	}

	FTextureRHIRef GetFrameTexture() const { return FrameTexture; }

	void SetFrameTexture(FTextureRHIRef InFrameTexture) { FrameTexture = InFrameTexture; }

private:
	FTextureRHIRef FrameTexture;
	bool bHasWrapperFormatData;
	WrapperFormatData WrapperData;
};
