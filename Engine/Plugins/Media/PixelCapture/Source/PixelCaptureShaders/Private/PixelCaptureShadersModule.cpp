// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"

class FPixelCaptureShadersModule : public IModuleInterface
{
public:
	virtual ~FPixelCaptureShadersModule() = default;

	virtual void StartupModule() override;
};

void FPixelCaptureShadersModule::StartupModule()
{
	FString ShaderDirectory = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("PixelCapture"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/PixelCapture"), ShaderDirectory);
}

IMPLEMENT_MODULE(FPixelCaptureShadersModule, PixelCaptureShaders)
