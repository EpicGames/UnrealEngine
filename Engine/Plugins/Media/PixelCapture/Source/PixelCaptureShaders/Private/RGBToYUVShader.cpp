// Copyright Epic Games, Inc. All Rights Reserved.

#include "RGBToYUVShader.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "RenderGraphUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

#define NUM_THREADS_PER_GROUP_DIMENSION 32

class FExtractI420CS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FExtractI420CS);
	SHADER_USE_PARAMETER_STRUCT(FExtractI420CS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
	SHADER_PARAMETER(FVector2f, SourceDimensions)
	SHADER_PARAMETER_UAV(RWTexture2D<float>, OutputY)
	SHADER_PARAMETER_UAV(RWTexture2D<float>, OutputU)
	SHADER_PARAMETER_UAV(RWTexture2D<float>, OutputV)
	END_SHADER_PARAMETER_STRUCT()

	//Called by the engine to determine which permutations to compile for this shader
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
	//Modifies the compilations environment of the shader
	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		//We're using it here to add some preprocessor defines. That way we don't have to change both C++ and HLSL code when we change the value for NUM_THREADS_PER_GROUP_DIMENSION
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), NUM_THREADS_PER_GROUP_DIMENSION);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), NUM_THREADS_PER_GROUP_DIMENSION);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Z"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FExtractI420CS, "/Plugin/PixelCapture/Private/RGBToYUV.usf", "ExtractI420", SF_Compute);

void FRGBToYUVShader::Dispatch(FRHICommandListImmediate& RHICmdList, const FRGBToYUVShaderParameters& InParameters)
{
	FExtractI420CS::FParameters PassParametersY;
	PassParametersY.SourceTexture = InParameters.SourceTexture;
	PassParametersY.SourceSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParametersY.SourceDimensions = { (float)InParameters.DestPlaneYDimensions.X, (float)InParameters.DestPlaneYDimensions.Y };
	PassParametersY.OutputY = InParameters.DestPlaneY;
	PassParametersY.OutputU = InParameters.DestPlaneU;
	PassParametersY.OutputV = InParameters.DestPlaneV;

	TShaderMapRef<FExtractI420CS> ComputeShaderY(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShaderY, PassParametersY,
		FIntVector(FMath::DivideAndRoundUp(InParameters.DestPlaneYDimensions.X, NUM_THREADS_PER_GROUP_DIMENSION),
			FMath::DivideAndRoundUp(InParameters.DestPlaneYDimensions.Y, NUM_THREADS_PER_GROUP_DIMENSION), 1));
}
