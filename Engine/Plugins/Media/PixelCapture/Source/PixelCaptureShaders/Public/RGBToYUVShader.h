// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "RHIResources.h"
#include "RHICommandList.h"
#include "RHIStaticStates.h"

#define UE_API PIXELCAPTURESHADERS_API

struct FRGBToYUVShaderParameters
{
	FTextureRHIRef SourceTexture;

	FIntPoint DestPlaneYDimensions;
	FIntPoint DestPlaneUVDimensions;

	FUnorderedAccessViewRHIRef DestPlaneY;
	FUnorderedAccessViewRHIRef DestPlaneU;
	FUnorderedAccessViewRHIRef DestPlaneV;
};

class FRGBToYUVShader
{
public:
	static UE_API void Dispatch(FRHICommandListImmediate& RHICmdList, const FRGBToYUVShaderParameters& InParameters);
};

#undef UE_API
